// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSimplify.h"
#include "Math/RandomStream.h"

FMeshSimplifier::FMeshSimplifier( float* InVerts, uint32 InNumVerts, uint32* InIndexes, uint32 InNumIndexes, int32* InMaterialIndexes, uint32 InNumAttributes )
	: NumVerts( InNumVerts )
	, NumIndexes( InNumIndexes )
	, NumAttributes( InNumAttributes )
	, NumTris( NumIndexes / 3 )
	, RemainingNumVerts( NumVerts )
	, RemainingNumTris( NumTris )
	, Verts( InVerts )
	, Indexes( InIndexes )
	, MaterialIndexes( InMaterialIndexes )
	, VertHash( 1 << FMath::Min( 16u, FMath::FloorLog2( NumVerts ) ) )
	, CornerHash( 1 << FMath::Min( 16u, FMath::FloorLog2( NumIndexes ) ) )
	, TriRemoved( false, NumTris )
{
	for( uint32 VertIndex = 0; VertIndex < NumVerts; VertIndex++ )
	{
		VertHash.Add( HashPosition( GetPosition( VertIndex ) ), VertIndex );
	}

	VertRefCount.AddZeroed( NumVerts );
	CornerFlags.AddZeroed( NumIndexes );
	
	EdgeQuadrics.AddUninitialized( NumIndexes );
	EdgeQuadricsValid.Init( false, NumIndexes );

	// Guess number of edges based on Euler's formula.
	uint32 NumEdges = FMath::Min3( NumIndexes, 3 * NumVerts - 6, NumTris + NumVerts );
	Pairs.Reserve( NumEdges );
	PairHash0.Clear( 1 << FMath::Min( 16u, FMath::FloorLog2( NumEdges ) ), NumEdges );
	PairHash1.Clear( 1 << FMath::Min( 16u, FMath::FloorLog2( NumEdges ) ), NumEdges );

	for( uint32 Corner = 0; Corner < NumIndexes; Corner++ )
	{
		uint32 VertIndex = Indexes[ Corner ];
		
		VertRefCount[ VertIndex ]++;

		const FVector3f& Position = GetPosition( VertIndex );
		CornerHash.Add( HashPosition( Position ), Corner );

		uint32 OtherCorner = Cycle3( Corner );

		FPair Pair;
		Pair.Position0 = Position;
		Pair.Position1 = GetPosition( Indexes[ Cycle3( Corner ) ] );

		if( AddUniquePair( Pair, Pairs.Num() ) )
		{
			Pairs.Add( Pair );
		}
	}
}

void FMeshSimplifier::LockPosition( const FVector3f& Position )
{
	ForAllCorners( Position,
		[ this ]( uint32 Corner )
		{
			CornerFlags[ Corner ] |= LockedVertMask;
		} );
}

bool FMeshSimplifier::AddUniquePair( FPair& Pair, uint32 PairIndex )
{
	uint32 Hash0 = HashPosition( Pair.Position0 );
	uint32 Hash1 = HashPosition( Pair.Position1 );

	if( Hash0 > Hash1 )
	{
		Swap( Hash0, Hash1 );
		Swap( Pair.Position0, Pair.Position1 );
	}

	uint32 OtherPairIndex;
	for( OtherPairIndex = PairHash0.First( Hash0 ); PairHash0.IsValid( OtherPairIndex ); OtherPairIndex = PairHash0.Next( OtherPairIndex ) )
	{
		check( PairIndex != OtherPairIndex );

		FPair& OtherPair = Pairs[ OtherPairIndex ];

		if( Pair.Position0 == OtherPair.Position0 &&
			Pair.Position1 == OtherPair.Position1 )
		{
			// Found a duplicate
			return false;
		}
	}

	PairHash0.Add( Hash0, PairIndex );
	PairHash1.Add( Hash1, PairIndex );

	return true;
}

void FMeshSimplifier::CalcTriQuadric( uint32 TriIndex )
{
	uint32 i0 = Indexes[ TriIndex * 3 + 0 ];
	uint32 i1 = Indexes[ TriIndex * 3 + 1 ];
	uint32 i2 = Indexes[ TriIndex * 3 + 2 ];

	new( &GetTriQuadric( TriIndex ) ) FQuadricAttr(
		GetPosition( i0 ),
		GetPosition( i1 ),
		GetPosition( i2 ),
		GetAttributes( i0 ),
		GetAttributes( i1 ),
		GetAttributes( i2 ),
		AttributeWeights,
		NumAttributes );
}

void FMeshSimplifier::CalcEdgeQuadric( uint32 EdgeIndex )
{
	uint32 TriIndex = EdgeIndex / 3;
	if( TriRemoved[ TriIndex ] )
	{
		EdgeQuadricsValid[ EdgeIndex ] = false;
		return;
	}

	int32 MaterialIndex = MaterialIndexes[ TriIndex ];

	uint32 VertIndex0 = Indexes[ EdgeIndex ];
	uint32 VertIndex1 = Indexes[ Cycle3( EdgeIndex ) ];

	const FVector3f& Position0 = GetPosition( VertIndex0 );
	const FVector3f& Position1 = GetPosition( VertIndex1 );
	
	// Find edge with opposite direction that shares these 2 verts.
	// If none then we need to add an edge constraint.
	/*
		  /\
		 /  \
		o-<<-o
		o->>-o
		 \  /
		  \/
	*/
	uint32 Hash = HashPosition( Position1 );
	uint32 Corner;
	for( Corner = CornerHash.First( Hash ); CornerHash.IsValid( Corner ); Corner = CornerHash.Next( Corner ) )
	{
		uint32 OtherVertIndex0 = Indexes[ Corner ];
		uint32 OtherVertIndex1 = Indexes[ Cycle3( Corner ) ];
			
		if( VertIndex0 == OtherVertIndex1 &&
			VertIndex1 == OtherVertIndex0 &&
			MaterialIndex == MaterialIndexes[ Corner / 3 ] )
		{
			// Found matching edge.
			// No constraints needed so remove any that exist.
			EdgeQuadricsValid[ EdgeIndex ] = false;
			return;
		}
	}

	// Don't double count attribute discontinuities.
	float Weight = EdgeWeight;
	for( Corner = CornerHash.First( Hash ); CornerHash.IsValid( Corner ); Corner = CornerHash.Next( Corner ) )
	{
		uint32 OtherVertIndex0 = Indexes[ Corner ];
		uint32 OtherVertIndex1 = Indexes[ Cycle3( Corner ) ];
			
		if( Position0 == GetPosition( OtherVertIndex1 ) &&
			Position1 == GetPosition( OtherVertIndex0 ) )
		{
			// Found matching edge.
			Weight *= 0.5f;
			break;
		}
	}

	// Didn't find matching edge. Add edge constraint.
	EdgeQuadrics[ EdgeIndex ] = FEdgeQuadric( GetPosition( VertIndex0 ), GetPosition( VertIndex1 ), Weight );
	EdgeQuadricsValid[ EdgeIndex ] = true;
}

float FMeshSimplifier::EvaluateMerge( const FVector3f& Position0, const FVector3f& Position1, bool bMoveVerts )
{
	//check( Position0 != Position1 );
	if( Position0 == Position1 )
		return 0.0f;

	WedgeDisjointSet.Reset();

	// Find unique adjacent triangles
	TArray< uint32, TInlineAllocator<16> > AdjTris;

	struct FWedgeVert
	{
		uint32 VertIndex;
		uint32 AdjTriIndex;
	};
	TArray< FWedgeVert, TInlineAllocator<16> > WedgeVerts[2];

	int32 VertDegree = 0;

	auto GatherAdjTris = [ this, &AdjTris, &WedgeVerts, &VertDegree ]( const FVector3f& Position, uint32 Index, uint32& FlagsUnion )
	{
		ForAllCorners( Position,
			[ this, &AdjTris, &WedgeVerts, &VertDegree, Index, &FlagsUnion ]( uint32 Corner )
			{
				VertDegree++;
				
				{
					uint8& RESTRICT CornerFlag = CornerFlags[ Corner ];
					FlagsUnion |= CornerFlag;
					CornerFlag |= 1 << Index;
				}
				
				uint32 TriIndex = Corner / 3;
				uint32 AdjTriIndex;
				bool bNewTri = true;
				
				uint8& RESTRICT FirstCornerFlag = CornerFlags[ TriIndex * 3 ];
				if( ( FirstCornerFlag & AdjTriMask ) == 0 )
				{
					FirstCornerFlag |= AdjTriMask;
					AdjTriIndex = AdjTris.Add( TriIndex );
					WedgeDisjointSet.AddDefaulted();
				}
				else
				{
					// Should only happen 2 times per collapse on average
					AdjTriIndex = AdjTris.Find( TriIndex );
					bNewTri = false;
				}
		
				uint32 VertIndex = Indexes[ Corner ];
				uint32 OtherAdjTriIndex = ~0u;
				for( FWedgeVert& WedgeVert : WedgeVerts[ Index ] )
				{
					if( VertIndex == WedgeVert.VertIndex )
					{
						OtherAdjTriIndex = WedgeVert.AdjTriIndex;
						break;
					}
				}
				if( OtherAdjTriIndex == ~0u )
				{
					WedgeVerts[ Index ].Add( { VertIndex, AdjTriIndex } );
				}
				else
				{
					if( bNewTri )
						WedgeDisjointSet.UnionSequential( AdjTriIndex, OtherAdjTriIndex );
					else
						WedgeDisjointSet.Union( AdjTriIndex, OtherAdjTriIndex );
				}
			} );
	};

	uint32 FlagsUnion0 = 0;
	uint32 FlagsUnion1 = 0;

	GatherAdjTris( Position0, 0, FlagsUnion0 );
	GatherAdjTris( Position1, 1, FlagsUnion1 );

	if( VertDegree == 0 )
	{
		return 0.0f;
	}

	// This would mean this collapse will remove all remaining triangles.
	if( VertDegree == RemainingNumTris * 2 )
	{
		// Clean up corner flags
		check( AdjTris.Num() > 0 );
		for( uint32 TriIndex : AdjTris )
		{
			for( uint32 CornerIndex = 0; CornerIndex < 3; CornerIndex++ )
			{
				CornerFlags[ TriIndex * 3 + CornerIndex ] &= ~( MergeMask | AdjTriMask );
			}
		}
		return 0.0f;
	}

	bool bLocked0 = FlagsUnion0 & LockedVertMask;
	bool bLocked1 = FlagsUnion1 & LockedVertMask;

	float Penalty = 0.0f;

	if( VertDegree > DegreeLimit )
		Penalty += DegreePenalty * ( VertDegree - DegreeLimit );
	
	TArray< uint32, TInlineAllocator<8> >	WedgeIDs;
	TArray< uint8, TInlineAllocator<1024> >	WedgeQuadrics;
	
	const SIZE_T QuadricSize = sizeof( FQuadricAttr ) + NumAttributes * 4 * sizeof( QScalar );

	auto GetWedgeQuadric =
		[ &WedgeQuadrics, QuadricSize ]( int32 WedgeIndex ) -> FQuadricAttr&
		{
			return *reinterpret_cast< FQuadricAttr* >( &WedgeQuadrics[ WedgeIndex * QuadricSize ] );
		};

	for( uint32 AdjTriIndex = 0, Num = AdjTris.Num(); AdjTriIndex < Num; AdjTriIndex++ )
	{
		uint32 TriIndex = AdjTris[ AdjTriIndex ];

		FQuadricAttr& RESTRICT TriQuadric = GetTriQuadric( TriIndex );

		uint32 WedgeID = WedgeDisjointSet.Find( AdjTriIndex );
		int32 WedgeIndex = WedgeIDs.Find( WedgeID );
		if( WedgeIndex != INDEX_NONE )
		{
			FQuadricAttr& RESTRICT WedgeQuadric = GetWedgeQuadric( WedgeIndex );

#if SIMP_REBASE
			uint32 VertIndex0 = Indexes[ TriIndex * 3 ];
			WedgeQuadric.Add( TriQuadric, GetPosition( VertIndex0 ) - Position0, GetAttributes( VertIndex0 ), AttributeWeights, NumAttributes );
#else
			WedgeQuadric.Add( TriQuadric, NumAttributes );
#endif
		}
		else
		{
			WedgeIndex = WedgeIDs.Add( WedgeID );
			WedgeQuadrics.AddUninitialized( QuadricSize );

			FQuadricAttr& RESTRICT WedgeQuadric = GetWedgeQuadric( WedgeIndex );

			FMemory::Memcpy( &WedgeQuadric, &TriQuadric, QuadricSize );
#if SIMP_REBASE
			uint32 VertIndex0 = Indexes[ TriIndex * 3 ];
			WedgeQuadric.Rebase( GetPosition( VertIndex0 ) - Position0, GetAttributes( VertIndex0 ), AttributeWeights, NumAttributes );
#endif
		}
	}

	FQuadricAttrOptimizer QuadricOptimizer;
	for( int32 WedgeIndex = 0, Num = WedgeIDs.Num(); WedgeIndex < Num; WedgeIndex++ )
	{
		QuadricOptimizer.AddQuadric( GetWedgeQuadric( WedgeIndex ), NumAttributes );
	}

	FVector3f	BoundsMin = {  MAX_flt,  MAX_flt,  MAX_flt };
	FVector3f	BoundsMax = { -MAX_flt, -MAX_flt, -MAX_flt };

	FQuadric EdgeQuadric;
	EdgeQuadric.Zero();

	for( uint32 TriIndex : AdjTris )
	{
		for( uint32 CornerIndex = 0; CornerIndex < 3; CornerIndex++ )
		{
			uint32 Corner = TriIndex * 3 + CornerIndex;

			const FVector3f& Position = GetPosition( Indexes[ Corner ] );

			BoundsMin = FVector3f::Min( BoundsMin, Position );
			BoundsMax = FVector3f::Max( BoundsMax, Position );
			
			if( EdgeQuadricsValid[ Corner ] )
			{
				// Only if edge is part of this pair
				uint32 EdgeFlags;
				EdgeFlags  = CornerFlags[ Corner ];
				EdgeFlags |= CornerFlags[ TriIndex * 3 + ( ( 1 << CornerIndex ) & 3 ) ];
				if( EdgeFlags & MergeMask )
				{
#if SIMP_REBASE
					EdgeQuadric.Add( EdgeQuadrics[ Corner ], GetPosition( Indexes[ Corner ] ) - Position0 );
#else
					uint32 VertIndex0 = Indexes[ Corner ];
					uint32 VertIndex1 = Indexes[ Cycle3( Corner ) ];
					//EdgeQuadric += FQuadric( GetPosition( VertIndex0 ), GetPosition( VertIndex1 ), GetNormal( TriIndex ), EdgeWeight );
					EdgeQuadric.Add( EdgeQuadrics[ Corner ], GetPosition( Indexes[ Corner ] ) - QuadricOrigin );
#endif
				}
			}
		}
	}

	QuadricOptimizer.AddQuadric( EdgeQuadric );
	
	auto IsValidPosition =
		[ this, &AdjTris, &BoundsMin, &BoundsMax ]( const FVector3f& Position ) -> bool
		{
			// Limit position to be near the neighborhood bounds
			if( ComputeSquaredDistanceFromBoxToPoint( BoundsMin, BoundsMax, Position ) > ( BoundsMax - BoundsMin ).SizeSquared() * 4.0f )
				return false;

			for( uint32 TriIndex : AdjTris )
			{
				if( TriWillInvert( TriIndex, Position ) )
					return false;
			}

			return true;
		};

	FVector3f NewPosition;
	{
		if( bLocked0 && bLocked1 )
			Penalty += LockPenalty;

		// find position
		if( bLocked0 && !bLocked1 )
		{
			NewPosition = Position0;

			if( !IsValidPosition( NewPosition ) )
				Penalty += InversionPenalty;
		}
		else if( bLocked1 && !bLocked0 )
		{
			NewPosition = Position1;

			if( !IsValidPosition( NewPosition ) )
				Penalty += InversionPenalty;
		}
		else
		{
			bool bIsValid = QuadricOptimizer.OptimizeVolume( NewPosition );
#if SIMP_REBASE
			NewPosition += Position0;
#endif
			if( bIsValid )
				bIsValid = IsValidPosition( NewPosition );

			if( !bIsValid )
			{
				bIsValid = QuadricOptimizer.Optimize( NewPosition );
#if SIMP_REBASE
				NewPosition += Position0;
#endif
				if( bIsValid )
					bIsValid = IsValidPosition( NewPosition );
			}
			
			if( !bIsValid )
			{
				// Try a point on the edge.
#if SIMP_REBASE
				bIsValid = QuadricOptimizer.OptimizeLinear( FVector3f::ZeroVector, Position1 - Position0, NewPosition );
				NewPosition += Position0;
#else
				bIsValid = QuadricOptimizer.OptimizeLinear( Position0, Position1, NewPosition );
#endif
				if( bIsValid )
					bIsValid = IsValidPosition( NewPosition );
			}

			if( !bIsValid )
			{
				// Couldn't find optimal so choose middle
				NewPosition = ( Position0 + Position1 ) * 0.5f;

				if( !IsValidPosition( NewPosition ) )
					Penalty += InversionPenalty;
			}
		}
	}

	int32 NumWedges = WedgeIDs.Num();
	WedgeAttributes.Reset();
	WedgeAttributes.AddUninitialized( NumWedges * NumAttributes );

#if SIMP_REBASE
	FVector3f NewPositionRebase = NewPosition - Position0;
#else
	FVector3f& NewPositionRebase = NewPosition;
#endif

	float Error = 0.0f;
	float EdgeError = EdgeQuadric.Evaluate( NewPositionRebase );
	float SurfaceArea = 0.0f;

	if( bLocked0 != bLocked1 || bZeroWeights )
	{
		const float DistSqr0 = FVector3f::DistSquared( NewPosition, Position0 );
		const float DistSqr1 = FVector3f::DistSquared( NewPosition, Position1 );
	
		// Start with farthest so that the second pass with closest will overwrite any verts from the same wedge.
		// Can't do only the closest since there may be wedges only present in the farthest.
		uint32 Farthest = DistSqr0 > DistSqr1 ? 0 : 1;

		for( uint32 j = 0; j < 2; j++ )
		{
			for( FWedgeVert& WedgeVert : WedgeVerts[ ( Farthest + j ) & 1 ] )
			{
				int32 WedgeIndex = WedgeIDs.Find( WedgeDisjointSet[ WedgeVert.AdjTriIndex ] );

				float* RESTRICT NewAttributes = &WedgeAttributes[ WedgeIndex * NumAttributes ];
				float* RESTRICT OldAttributes = GetAttributes( WedgeVert.VertIndex );

				for( uint32 i = 0; i < NumAttributes; i++ )
					NewAttributes[i] = OldAttributes[i];
			}
		}
	}

	for( int32 WedgeIndex = 0; WedgeIndex < NumWedges; WedgeIndex++ )
	{
		float* RESTRICT NewAttributes = &WedgeAttributes[ WedgeIndex * NumAttributes ];

		FQuadricAttr& RESTRICT WedgeQuadric = GetWedgeQuadric( WedgeIndex );
		if( WedgeQuadric.a > 1e-8 )
		{
			float WedgeError;
			if( bLocked0 != bLocked1 )
			{
				WedgeError = WedgeQuadric.Evaluate( NewPositionRebase, NewAttributes, AttributeWeights, NumAttributes );
			}
			else
			{
				// calculate vert attributes from the new position
				WedgeError = WedgeQuadric.CalcAttributesAndEvaluate( NewPositionRebase, NewAttributes, AttributeWeights, NumAttributes );
				
				// Correct after eval. Normal length is unimportant for error but can bias the calculation.
				if( CorrectAttributes != nullptr )
					CorrectAttributes( NewAttributes );
			}

			if( bLimitErrorToSurfaceArea )
				WedgeError = FMath::Min( WedgeError, WedgeQuadric.a );

			Error += WedgeError;
		}
		else
		{
			for( uint32 i = 0; i < NumAttributes; i++ )
			{
				NewAttributes[i] = 0.0f;
			}
		}
		
		SurfaceArea += WedgeQuadric.a;
	}

	Error += EdgeError;
	
	bool bIsDisjoint = AdjTris.Num() == 1 || ( AdjTris.Num() == 2 && VertDegree == 4 );

	if( bLimitErrorToSurfaceArea )
	{
		// Limit error to be no greater than the size of the triangles it could affect.
		Error = FMath::Min( Error, SurfaceArea );
		
		// Collapsing will completely remove this surface area. The position merged to is irrelevant.
		if( bIsDisjoint )
			Error = SurfaceArea;
	}

	// Check to set error based on edge length
	if( MaxEdgeLengthFactor > 0.0f )
	{
		for( uint32 TriIndex : AdjTris )
		{
			uint32 IndexMoved = CornerIndexMoved( TriIndex );

			if( IndexMoved < 3 )
			{
				uint32 Corner = TriIndex * 3 + IndexMoved;

				const FVector3f& p1 = GetPosition( Indexes[ Cycle3( Corner ) ] );
				const FVector3f& p2 = GetPosition( Indexes[ Cycle3( Corner, 2 ) ] );

				Error = FMath::Max( Error, ( NewPosition - p1 ).SizeSquared() / ( MaxEdgeLengthFactor * MaxEdgeLengthFactor ) );
				Error = FMath::Max( Error, ( NewPosition - p2 ).SizeSquared() / ( MaxEdgeLengthFactor * MaxEdgeLengthFactor ) );
			}
		}
	}

	if( bMoveVerts )
	{
		BeginMovePosition( Position0 );
		BeginMovePosition( Position1 );

		for( uint32 AdjTriIndex = 0, Num = AdjTris.Num(); AdjTriIndex < Num; AdjTriIndex++ )
		{
			int32 WedgeIndex = WedgeIDs.Find( WedgeDisjointSet[ AdjTriIndex ] );

			for( uint32 CornerIndex = 0; CornerIndex < 3; CornerIndex++ )
			{
				uint32 Corner = AdjTris[ AdjTriIndex ] * 3 + CornerIndex;
				uint32 VertIndex = Indexes[ Corner ];

				FVector3f& OldPosition = GetPosition( VertIndex );
				if( OldPosition == Position0 ||
					OldPosition == Position1 )
				{
					OldPosition = NewPosition;

					// Only use attributes if we calculated them.
					if( GetWedgeQuadric( WedgeIndex ).a > 1e-8 )
					{
						float* RESTRICT NewAttributes = &WedgeAttributes[ WedgeIndex * NumAttributes ];
						float* RESTRICT OldAttributes = GetAttributes( VertIndex );

						for( uint32 i = 0; i < NumAttributes; i++ )
						{
							OldAttributes[i] = NewAttributes[i];
						}
					}
					
					// If either position was locked then lock the new verts.
					if( bLocked0 || bLocked1 )
						CornerFlags[ Corner ] |= LockedVertMask;
				}
			}
		}

		for( uint32 PairIndex : MovedPairs )
		{
			FPair& Pair = Pairs[ PairIndex ];

			checkSlow( Pair.Position0 != Position0 || Pair.Position1 != Position1 );

			if( Pair.Position0 == Position0 ||
				Pair.Position0 == Position1 )
			{
				Pair.Position0 = NewPosition;
			}

			if( Pair.Position1 == Position0 ||
				Pair.Position1 == Position1 )
			{
				Pair.Position1 = NewPosition;
			}
		}

		EndMovePositions();

		TArray< uint32, TInlineAllocator<16> > AdjVerts;
		for( uint32 TriIndex : AdjTris )
		{
			for( uint32 CornerIndex = 0; CornerIndex < 3; CornerIndex++ )
			{
				AdjVerts.AddUnique( Indexes[ TriIndex * 3 + CornerIndex ] );
			}
		}

		// Reevaluate all pairs touching an adjacent tri.
		// Duplicate pairs have already been removed.
		for( uint32 VertIndex : AdjVerts )
		{
			const FVector3f& Position = GetPosition( VertIndex );

			ForAllPairs( Position,
				[ this ]( uint32 PairIndex )
				{
					// IsPresent used to mark Pairs we have already added to the list.
					if( PairHeap.IsPresent( PairIndex ) )
					{
						PairHeap.Remove( PairIndex );
						ReevaluatePairs.Add( PairIndex );
					}
				} );
		}

		for( uint32 TriIndex : AdjTris )
		{
			int32 MaterialIndex = MaterialIndexes[ TriIndex ] & 0xffffff;
			if( !PerMaterialDeltas.IsValidIndex( MaterialIndex ) )
				PerMaterialDeltas.SetNumZeroed( MaterialIndex + 1 );

			auto& Delta = PerMaterialDeltas[ MaterialIndex ];

			Delta.SurfaceArea -= GetTriQuadric( TriIndex ).a;
			Delta.NumTris--;
			Delta.NumDisjoint -= bIsDisjoint ? 1 : 0;

			FixUpTri( TriIndex );
			
			if( !TriRemoved[ TriIndex ] )
			{
				Delta.SurfaceArea += GetTriQuadric( TriIndex ).a;
				Delta.NumTris++;
			}
		}
	}
	else
	{
		Error += Penalty;
	}

	for( uint32 TriIndex : AdjTris )
	{
		for( uint32 CornerIndex = 0; CornerIndex < 3; CornerIndex++ )
		{
			uint32 Corner = TriIndex * 3 + CornerIndex;

			// Must be separated from FixUpTri loop because relies on correct indexing
			if( bMoveVerts )
				CalcEdgeQuadric( Corner );

			// Clear flags
			CornerFlags[ Corner ] &= ~( MergeMask | AdjTriMask );
		}
	}

	return Error;
}

void FMeshSimplifier::BeginMovePosition( const FVector3f& Position )
{
	uint32 Hash = HashPosition( Position );

	ForAllVerts( Position, 
		[ this, Hash ]( uint32 VertIndex )
		{
			// Safe to remove while iterating.
			VertHash.Remove( Hash, VertIndex );
			MovedVerts.Add( VertIndex );
		} );

	ForAllCorners( Position, 
		[ this, Hash ]( uint32 Corner )
		{
			// Safe to remove while iterating.
			CornerHash.Remove( Hash, Corner );
			MovedCorners.Add( Corner );
		} );

	ForAllPairs( Position,
		[ this ]( uint32 PairIndex )
		{
			// Safe to remove while iterating.
			PairHash0.Remove( HashPosition( Pairs[ PairIndex ].Position0 ), PairIndex );
			PairHash1.Remove( HashPosition( Pairs[ PairIndex ].Position1 ), PairIndex );
			MovedPairs.Add( PairIndex );
		} );
}

void FMeshSimplifier::EndMovePositions()
{
	for( uint32 VertIndex : MovedVerts )
	{
		VertHash.Add( HashPosition( GetPosition( VertIndex ) ), VertIndex );
	}

	for( uint32 Corner : MovedCorners )
	{
		CornerHash.Add( HashPosition( GetPosition( Indexes[ Corner ] ) ), Corner );
	}

	for( uint32 PairIndex : MovedPairs )
	{
		FPair& Pair = Pairs[ PairIndex ];

		if( Pair.Position0 == Pair.Position1 || !AddUniquePair( Pair, PairIndex ) )
		{
			// Found invalid or duplicate pair
			PairHeap.Remove( PairIndex );
		}
	}

	MovedVerts.Reset();
	MovedCorners.Reset();
	MovedPairs.Reset();
}

uint32 FMeshSimplifier::CornerIndexMoved( uint32 TriIndex ) const
{
	uint32 IndexMoved = 3;
	for( uint32 CornerIndex = 0; CornerIndex < 3; CornerIndex++ )
	{
		uint32 Corner = TriIndex * 3 + CornerIndex;

		if( CornerFlags[ Corner ] & MergeMask )
		{
			if( IndexMoved == 3 )
				IndexMoved = CornerIndex;
			else
				IndexMoved = 4;
		}
	}
	return IndexMoved;
}

bool FMeshSimplifier::TriWillInvert( uint32 TriIndex, const FVector3f& NewPosition ) const
{
	uint32 IndexMoved = CornerIndexMoved( TriIndex );

	if( IndexMoved < 3 )
	{
		uint32 Corner = TriIndex * 3 + IndexMoved;

		const FVector3f& p0 = GetPosition( Indexes[ Corner ] );
		const FVector3f& p1 = GetPosition( Indexes[ Cycle3( Corner ) ] );
		const FVector3f& p2 = GetPosition( Indexes[ Cycle3( Corner, 2 ) ] );

		const FVector3f d21 = p2 - p1;
		const FVector3f d01 = p0 - p1;
		const FVector3f dp1 = NewPosition - p1;

	#if 1
		FVector3f n0 = d01 ^ d21;
		FVector3f n1 = dp1 ^ d21;

		return (n0 | n1) < 0.0f;
	#else
		FVector3f n = d21 ^ d01 ^ d21;
		//n.Normalize();

		float InversionThreshold = 0.0f;
		return (dp1 | n) < InversionThreshold * (d01 | n);
	#endif
	}

	return false;
}

void FMeshSimplifier::RemoveTri( uint32 TriIndex )
{
	check( !TriRemoved[ TriIndex ] );

	TriRemoved[ TriIndex ] = true;
	RemainingNumTris--;

	// Remove references to tri
	for( uint32 k = 0; k < 3; k++ )
	{
		uint32 Corner = TriIndex * 3 + k;
		uint32 VertIndex = Indexes[ Corner ];
		uint32 Hash = HashPosition( GetPosition( VertIndex ) );

		CornerHash.Remove( Hash, Corner );
		EdgeQuadricsValid[ Corner ] = false;

		SetVertIndex( Corner, ~0u );
	}
}

void FMeshSimplifier::FixUpTri( uint32 TriIndex )
{
	check( !TriRemoved[ TriIndex ] );

	const FVector3f& p0 = GetPosition( Indexes[ TriIndex * 3 + 0 ] );
	const FVector3f& p1 = GetPosition( Indexes[ TriIndex * 3 + 1 ] );
	const FVector3f& p2 = GetPosition( Indexes[ TriIndex * 3 + 2 ] );

	bool bRemoveTri = CornerFlags[ TriIndex * 3 ] & RemoveTriMask;

	if( !bRemoveTri )
	{
		// Remove degenerates
		bRemoveTri =
			p0 == p1 ||
			p1 == p2 ||
			p2 == p0;
	}

	if( !bRemoveTri )
	{
		for( uint32 k = 0; k < 3; k++ )
		{
			RemoveDuplicateVerts( TriIndex * 3 + k );
		}

		bRemoveTri = IsDuplicateTri( TriIndex );
	}

	if( bRemoveTri )
		RemoveTri( TriIndex );
	else
		CalcTriQuadric( TriIndex );
}

bool FMeshSimplifier::IsDuplicateTri( uint32 TriIndex ) const
{
	uint32 i0 = Indexes[ TriIndex * 3 + 0 ];
	uint32 i1 = Indexes[ TriIndex * 3 + 1 ];
	uint32 i2 = Indexes[ TriIndex * 3 + 2 ];

	uint32 Hash = HashPosition( GetPosition( i0 ) );
	for( uint32 Corner = CornerHash.First( Hash ); CornerHash.IsValid( Corner ); Corner = CornerHash.Next( Corner ) )
	{
		if( Corner != TriIndex * 3 &&
			i0 == Indexes[ Corner ] &&
			i1 == Indexes[ Cycle3( Corner ) ] &&
			i2 == Indexes[ Cycle3( Corner, 2 ) ] )
		{
			return true;
		}
	}

	return false;
}

void FMeshSimplifier::SetVertIndex( uint32 Corner, uint32 NewVertIndex )
{
	uint32& VertIndex = Indexes[ Corner ];
	checkSlow( VertIndex != ~0u );
	checkSlow( VertRefCount[ VertIndex ] > 0 );

	if( VertIndex == NewVertIndex )
		return;

	uint32 RefCount = --VertRefCount[ VertIndex ];
	if( RefCount == 0 )
	{
		VertHash.Remove( HashPosition( GetPosition( VertIndex ) ), VertIndex );
		RemainingNumVerts--;
	}

	VertIndex = NewVertIndex;
	if( VertIndex != ~0u )
	{
		VertRefCount[ VertIndex ]++;
	}
}

// Remove identical valued verts.
void FMeshSimplifier::RemoveDuplicateVerts( uint32 Corner )
{
	uint32& VertIndex = Indexes[ Corner ];
	float* VertData = &Verts[ ( 3 + NumAttributes ) * VertIndex ];

	uint32 Hash = HashPosition( GetPosition( VertIndex ) );
	for( uint32 OtherVertIndex = VertHash.First( Hash ); VertHash.IsValid( OtherVertIndex ); OtherVertIndex = VertHash.Next( OtherVertIndex ) )
	{
		if( VertIndex == OtherVertIndex )
			break;
		
		const uint32 NumFloats = 3 + NumAttributes;
		float* OtherVertData = &Verts[ NumFloats * OtherVertIndex ];

		uint32 i;
		for( i = 0; i < NumFloats; i++ )
		{
			if ( VertData[ i ] != OtherVertData[ i ] )
				break;
		}

		if( i == NumFloats )
		{
			// First entry in hashtable for this vert value is authoritative.
			SetVertIndex( Corner, OtherVertIndex );
			break;
		}
	}
}

void FMeshSimplifier::CaptureBackupTriangle( uint32 TriIndex, float Area )
{
	const uint32 NumFloats = 3 + NumAttributes;

	if( BackupVertData.Num() != static_cast< int32 >( 3 * NumFloats ) )
	{
		BackupVertData.SetNumUninitialized( 3 * NumFloats );
	}

	for( uint32 k = 0; k < 3; k++ )
	{
		const uint32 VertIndex = Indexes[ TriIndex * 3 + k ];
		const float* SrcVertData = &Verts[ NumFloats * VertIndex ];
		float* DstVertData = &BackupVertData[ NumFloats * k ];
		FMemory::Memcpy( DstVertData, SrcVertData, NumFloats * sizeof( float ) );
	}

	BackupMaterialIndex = MaterialIndexes[ TriIndex ];
	BackupArea = Area;
	bBackupValid = true;
}

void FMeshSimplifier::RestoreBackupTriangle( float& InOutMaxError )
{
	if( !bBackupValid )
	{
		return;
	}

	const uint32 NumFloats = 3 + NumAttributes;

	for( uint32 k = 0; k < 3; k++ )
	{
		float* DstVertData = &Verts[ NumFloats * k ];
		const float* SrcVertData = &BackupVertData[ NumFloats * k ];
		FMemory::Memcpy( DstVertData, SrcVertData, NumFloats * sizeof( float ) );

		VertRefCount[ k ] = 1;
		Indexes[ k ] = k;
	}

	MaterialIndexes[ 0 ] = BackupMaterialIndex;
	TriRemoved[ 0 ] = false;

	RemainingNumVerts = 3;
	RemainingNumTris = 1;

	CalcTriQuadric( 0 );

	const float LostArea = FMath::Max( 0.0f, TotalInputArea - BackupArea );
	InOutMaxError = FMath::Max( InOutMaxError, LostArea );
}

float FMeshSimplifier::Simplify(
	uint32 TargetNumVerts, uint32 TargetNumTris, float TargetError,
	uint32 LimitNumVerts, uint32 LimitNumTris, float LimitError )
{
	check( TargetNumVerts < NumVerts || TargetNumTris < NumTris || TargetError > 0.0f );
	check( TargetNumVerts >= LimitNumVerts );
	check( TargetNumTris >= LimitNumTris );
	check( TargetError <= LimitError );

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		if( AttributeWeights[i] == 0.0f )
		{
			bZeroWeights = true;
			break;
		}
	}

	const SIZE_T QuadricSize = sizeof( FQuadricAttr ) + NumAttributes * 4 * sizeof( QScalar );

	TriQuadrics.AddUninitialized( NumTris * QuadricSize );
	for( uint32 TriIndex = 0; TriIndex < NumTris; TriIndex++ )
	{
		FixUpTri( TriIndex );

		if( !TriRemoved[ TriIndex ] )
		{
			const float Area = GetTriQuadric( TriIndex ).a;
			TotalInputArea += Area;
			if( !bBackupValid || Area > BackupArea )
			{
				CaptureBackupTriangle( TriIndex, Area );
			}
		}
	}

	for( uint32 i = 0; i < NumIndexes; i++ )
	{
		CalcEdgeQuadric(i);
	}

	// Initialize heap
	PairHeap.Resize( Pairs.Num(), Pairs.Num() );
	
	for( uint32 PairIndex = 0, Num = Pairs.Num(); PairIndex < Num; PairIndex++ )
	{
		FPair& Pair = Pairs[ PairIndex ];

		float MergeError = EvaluateMerge( Pair.Position0, Pair.Position1, false );
		PairHeap.Add( MergeError, PairIndex );
	}

	float MaxError = 0.0f;

	while( PairHeap.Num() > 0 )
	{
		uint32 PrevNumVerts = RemainingNumVerts;
		uint32 PrevNumTris  = RemainingNumTris;

		if( PairHeap.GetKey( PairHeap.Top() ) > LimitError )
			break;

		{
			uint32 PairIndex = PairHeap.Top();
			PairHeap.Pop();

			FPair& Pair = Pairs[ PairIndex ];
		
			PairHash0.Remove( HashPosition( Pair.Position0 ), PairIndex );
			PairHash1.Remove( HashPosition( Pair.Position1 ), PairIndex );

			float MergeError = EvaluateMerge( Pair.Position0, Pair.Position1, true );
			MaxError = FMath::Max( MaxError, MergeError );
		}

		if( RemainingNumVerts	<= TargetNumVerts &&
			RemainingNumTris	<= TargetNumTris &&
			MaxError			>= TargetError )
		{
			break;
		}

		if( RemainingNumVerts	<= LimitNumVerts ||
			RemainingNumTris	<= LimitNumTris ||
			MaxError			>= LimitError )
		{
			break;
		}

		for( uint32 PairIndex : ReevaluatePairs )
		{
			FPair& Pair = Pairs[ PairIndex ];

			float MergeError = EvaluateMerge( Pair.Position0, Pair.Position1, false );
			PairHeap.Add( MergeError, PairIndex );
		}
		ReevaluatePairs.Reset();
	}

	// If couldn't hit targets through regular edge collapses, resort to randomly removing triangles.
	{
		uint32 TriIndex = 0;
		while(1)
		{
			if( RemainingNumVerts	<= TargetNumVerts &&
				RemainingNumTris	<= TargetNumTris &&
				MaxError			>= TargetError )
			{
				break;
			}
	
			if( RemainingNumVerts	<= LimitNumVerts ||
				RemainingNumTris	<= LimitNumTris ||
				MaxError			>= LimitError )
			{
				break;
			}

			while( TriRemoved[ TriIndex ] )
				TriIndex++;
	
			RemoveTri( TriIndex );
		}
	}

	if( bBackupValid && ( RemainingNumTris == 0 || RemainingNumVerts < 3 ) )
	{
		RestoreBackupTriangle( MaxError );
	}

	return MaxError;
}

void FMeshSimplifier::ShrinkTriGroupWithMostSurfaceAreaLoss(float ShrinkAmount)
{
	int32 ShrinkMaterialIndex = INDEX_NONE;
	float ShrinkSurfaceArea = 0.0f;
	for (int32 MaterialIndex = 0; MaterialIndex < PerMaterialDeltas.Num(); ++MaterialIndex)
	{
		if (PerMaterialDeltas[MaterialIndex].SurfaceArea < ShrinkSurfaceArea)
		{
			ShrinkMaterialIndex = MaterialIndex;
			ShrinkSurfaceArea = PerMaterialDeltas[MaterialIndex].SurfaceArea;
		}
	}

	if (ShrinkMaterialIndex == INDEX_NONE)
		return;

	constexpr uint32 NoCenterId = 0;
	constexpr uint32 CenterIdMask = 0x7fffffff;
	constexpr uint32 VertexLockedMask = 0x80000000;

	TArray<FVector4f> IslandCenters;
	TArray<uint32> VertToCenter;
	TArray<uint32> PendingTris;
	TBitArray<> TriVisited;

	VertToCenter.SetNumZeroed(NumVerts);
	TriVisited.Init(false, NumTris);

	auto ShouldVisitTri = [&](uint32 TriIndex)
	{
		return !TriVisited[TriIndex] && !TriRemoved[TriIndex] && (MaterialIndexes[TriIndex] & 0xffffff) == ShrinkMaterialIndex;
	};

	// Find all islands and accumulate vertex positions to compute averages later
	for (uint32 TriIndex = 0; TriIndex < NumTris; ++TriIndex)
	{
		if (!ShouldVisitTri(TriIndex))
		{
			continue;
		}

		// Start a new island
		const uint32 CenterId = IslandCenters.AddZeroed() + 1;
		FVector4f& CurCenter = IslandCenters.Last();

		PendingTris.Add(TriIndex);
		TriVisited[TriIndex] = true;

		while (PendingTris.Num() > 0)
		{
			const uint32 CurTriIndex = PendingTris.Pop(EAllowShrinking::No);

			for (uint32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
			{
				const uint32 EdgeIndex = CurTriIndex * 3 + CornerIndex;

				const uint32 VertIndex0 = Indexes[EdgeIndex];
				const uint32 VertIndex1 = Indexes[Cycle3(EdgeIndex)];

				const FVector3f& Position0 = GetPosition(VertIndex0);
				const FVector3f& Position1 = GetPosition(VertIndex1);

				const uint32 Hash = HashPosition(Position1);

				if (VertToCenter[VertIndex0] == NoCenterId)
				{
					// First time seeing this vertex. Mark it and accumulate position
					VertToCenter[VertIndex0] = CenterId;
					CurCenter += FVector4f(Position0, 1.f);
				}
				else if ((VertToCenter[VertIndex0] & CenterIdMask) != CenterId)
				{
					// Lock vertex shared by multiple islands. Is it better to split into two verts?
					VertToCenter[VertIndex0] = MAX_uint32;
				}

				if (CornerFlags[EdgeIndex] & LockedVertMask)
				{
					VertToCenter[VertIndex0] |= VertexLockedMask;
				}

				// Visit adjacent triangles if haven't already
				for (uint32 Corner = CornerHash.First(Hash); CornerHash.IsValid(Corner); Corner = CornerHash.Next(Corner))
				{
					const uint32 OtherVertIndex0 = Indexes[Corner];
					const uint32 OtherVertIndex1 = Indexes[Cycle3(Corner)];

					if (Position0 == GetPosition(OtherVertIndex1) && Position1 == GetPosition(OtherVertIndex0))
					{
						// Found matching edge.
						const uint32 OtherTriIndex = Corner / 3;

						if (ShouldVisitTri(OtherTriIndex))
						{
							PendingTris.Add(OtherTriIndex);
							TriVisited[OtherTriIndex] = true;
						}
					}
				}
			}
		}
	}

	// Compute average positions for all islands
	for (FVector4f& Center : IslandCenters)
	{
		Center.X /= Center.W;
		Center.Y /= Center.W;
		Center.Z /= Center.W;
	}

	// Lerp vertices to corresponding island centers
	for (uint32 VertIndex = 0; VertIndex < NumVerts; ++VertIndex)
	{
		const uint32 CenterId = VertToCenter[VertIndex];

		if (CenterId != NoCenterId && (CenterId & VertexLockedMask) == 0)
		{
			FVector3f& Position = GetPosition(VertIndex);
			const FVector3f Center(IslandCenters[CenterId - 1]);

			Position = FMath::Lerp(Position, Center, ShrinkAmount);
		}
	}
}

void FMeshSimplifier::ShrinkVoxelTriangles( float ShrinkAmount, TBitArray<>& InVoxelTriangles )
{
	for (TConstSetBitIterator<> BitIt( InVoxelTriangles ); BitIt; ++BitIt)
	{
		const uint32 TriIndex = BitIt.GetIndex();

		if (TriRemoved[ TriIndex ])
		{
			continue;
		}

		FVector3f Center = FVector3f::ZeroVector;

		for (uint32 k = 0; k < 3; k++)
		{
			const uint32 VertIndex = Indexes[ TriIndex * 3 + k ];
			Center += GetPosition( VertIndex );
		}

		Center /= 3;

		for (uint32 k = 0; k < 3; k++)
		{
			const uint32 VertIndex = Indexes[ TriIndex * 3 + k ];
			FVector3f& Position = GetPosition( VertIndex );
			Position = FMath::Lerp( Position, Center, ShrinkAmount );
		}
	}
}

void FMeshSimplifier::PreserveSurfaceArea()
{
	int32 DilateMaterialIndex = INDEX_NONE;
	float DilateSurfaceArea = 0.0f;
	for( int32 MaterialIndex = 0; MaterialIndex < PerMaterialDeltas.Num(); MaterialIndex++ )
	{
		if( PerMaterialDeltas[ MaterialIndex ].SurfaceArea < DilateSurfaceArea )
		{
			DilateMaterialIndex = MaterialIndex;
			DilateSurfaceArea = PerMaterialDeltas[ MaterialIndex ].SurfaceArea;
		}
	}

	if( DilateMaterialIndex == INDEX_NONE )
		return;

	TArray< FVector4f > EdgeNormals;
	EdgeNormals.AddZeroed( NumVerts );

	float Perimeter = 0.0f;
	float TotalArea = 0.0f;
	float ThisArea = 0.0f;

	uint32 NumEdges = 0;
	uint32 NumFaces = 0;

	for( uint32 TriIndex = 0; TriIndex < NumTris; TriIndex++ )
	{
		if( TriRemoved[ TriIndex ] )
			continue;

		float SurfaceArea = GetTriQuadric( TriIndex ).a;
		TotalArea += SurfaceArea;

		if( ( MaterialIndexes[ TriIndex ] & 0xffffff ) != DilateMaterialIndex )
			continue;

		ThisArea += SurfaceArea;
		NumFaces++;

		for( uint32 CornerIndex = 0; CornerIndex < 3; CornerIndex++ )
		{
			uint32 EdgeIndex = TriIndex * 3 + CornerIndex;
			
			if( EdgeQuadricsValid[ EdgeIndex ] )
			{
				uint32 VertIndex0 = Indexes[ EdgeIndex ];
				uint32 VertIndex1 = Indexes[ Cycle3( EdgeIndex ) ];

				const FVector3f& Position0 = GetPosition( VertIndex0 );
				const FVector3f& Position1 = GetPosition( VertIndex1 );

				// Find edge with opposite direction that shares these 2 verts.
				/*
					  /\
					 /  \
					o-<<-o
					o->>-o
					 \  /
					  \/
				*/
				uint32 Hash = HashPosition( Position1 );
				uint32 Corner;
				for( Corner = CornerHash.First( Hash ); CornerHash.IsValid( Corner ); Corner = CornerHash.Next( Corner ) )
				{
					uint32 OtherVertIndex0 = Indexes[ Corner ];
					uint32 OtherVertIndex1 = Indexes[ Cycle3( Corner ) ];
			
					if( Position0 == GetPosition( OtherVertIndex1 ) &&
						Position1 == GetPosition( OtherVertIndex0 ) )
					{
						// Found matching edge.
						break;
					}
				}
				if( !CornerHash.IsValid( Corner ) )
				{
					NumEdges++;

					const FVector3f& p0 = GetPosition( Indexes[ TriIndex * 3 + 0 ] );
					const FVector3f& p1 = GetPosition( Indexes[ TriIndex * 3 + 1 ] );
					const FVector3f& p2 = GetPosition( Indexes[ TriIndex * 3 + 2 ] );

					FVector3f Edge = Position1 - Position0;
					FVector3f FaceNormal = ( p2 - p0 ) ^ ( p1 - p0 );
					FVector3f EdgeNormal = FaceNormal ^ Edge;
					EdgeNormal.Normalize();

					float EdgeLength = Edge.Length();
					Perimeter  += EdgeLength;
					EdgeNormal *= EdgeLength;

					auto AddEdgeNormal = [&]( uint32 VertIndex )
					{
						EdgeNormals[ VertIndex ] += FVector4f( EdgeNormal, EdgeLength );
					};

					bool bLocked0 = CornerFlags[ EdgeIndex ]			& LockedVertMask;
					bool bLocked1 = CornerFlags[ Cycle3( EdgeIndex ) ]	& LockedVertMask;

					if( !bLocked0 ) ForAllVerts( Position0, AddEdgeNormal );
					if( !bLocked1 ) ForAllVerts( Position1, AddEdgeNormal );
				}
			}
		}
	}

	// If the scale is too big don't do it. Losing some area is better than suddenly getting a few gigantic leaves.
	if( -DilateSurfaceArea > 4.0f * ThisArea )
		return;

	// Does not accurately factor in corners.
	float DilateDistance = -DilateSurfaceArea / Perimeter;

	FRandomStream Random( NumEdges );

	for( uint32 VertIndex = 0; VertIndex < NumVerts; VertIndex++ )
	{
		FVector3f EdgeNormal = EdgeNormals[ VertIndex ];
		float Weight = EdgeNormals[ VertIndex ].W;
		if( Weight > 1e-6f )
		{
			//Scale * Perimeter / NumEdges = Weight * 0.5f;
			//float Scale = 2.0f * Perimeter / ( NumEdges * Weight );
			float Scale = Random.GetFraction() * 0.5f + 0.75f;
			EdgeNormal /= Weight;
			float LengthSqr = EdgeNormal.SizeSquared();
			if( LengthSqr > 0.1f )
			{
				GetPosition( VertIndex ) += EdgeNormal * ( Scale * DilateDistance / LengthSqr );
			}
		}
	}
}

void FMeshSimplifier::DumpOBJ( const char* Filename )
{
#if PLATFORM_WINDOWS
	FILE* File = nullptr;
	fopen_s(&File, Filename, "wb");
#else
	FILE* File = fopen(Filename, "wb");
#endif
	
	if( File )
	{
		for( uint32 i = 0; PairHeap.Num() > 0; i++ )
		{
			uint32 PairIndex = PairHeap.Top();

			float Error = PairHeap.GetKey( PairIndex );
			PairHeap.Pop();

			const FPair& Pair = Pairs[ PairIndex ];

			const FVector3f& p0 = Pair.Position0;
			const FVector3f& p1 = Pair.Position1;

			fprintf( File, "v %f %f %f\n", p0.X, p0.Y, p0.Z );
			fprintf( File, "v %f %f %f\n", p1.X, p1.Y, p1.Z );

			// +1 as Wavefront files are 1 index based
			fprintf( File, "l %u %u # %u %f\n", i*2 + 1, i*2 + 2, i + 1, Error );
		}
	
		fclose( File );
	}
}

void FMeshSimplifier::Compact()
{
	uint32 OutputVertIndex = 0;
	for( uint32 VertIndex = 0; VertIndex < NumVerts; VertIndex++ )
	{
		if( VertRefCount[ VertIndex ] > 0 )
		{
			if( VertIndex != OutputVertIndex )
			{
				float* SrcData = &Verts[ ( 3 + NumAttributes ) * VertIndex ];
				float* DstData = &Verts[ ( 3 + NumAttributes ) * OutputVertIndex ];
				FMemory::Memcpy( DstData, SrcData, ( 3 + NumAttributes ) * sizeof( float ) );
			}
			
			// Reuse VertRefCount as OutputVertIndex
			VertRefCount[ VertIndex ] = OutputVertIndex++;
		}
	}
	check( OutputVertIndex == RemainingNumVerts );
	
	uint32 OutputTriIndex = 0;
	for( uint32 TriIndex = 0; TriIndex < NumTris; TriIndex++ )
	{
		if( !TriRemoved[ TriIndex ] )
		{
			for( uint32 k = 0; k < 3; k++ )
			{
				// Reuse VertRefCount as OutputVertIndex
				uint32 VertIndex = Indexes[ TriIndex * 3 + k ];
				uint32 OutVertIndex = VertRefCount[ VertIndex ];
				Indexes[ OutputTriIndex * 3 + k ] = OutVertIndex;
			}

			MaterialIndexes[ OutputTriIndex++ ] = MaterialIndexes[ TriIndex ];
		}
	}
	check( OutputTriIndex == RemainingNumTris );
}