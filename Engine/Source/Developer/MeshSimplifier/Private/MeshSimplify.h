// Copyright (C) 2009 Nine Realms, Inc
//

#pragma once

#include "CoreMinimal.h"

#define SIMP_REBASE		1

#include "Quadric.h"
#include "Containers/HashTable.h"
#include "Containers/BinaryHeap.h"
#include "DisjointSet.h"

FORCEINLINE uint32 HashPosition( const FVector3f& Position )
{
	union { float f; uint32 i; } x;
	union { float f; uint32 i; } y;
	union { float f; uint32 i; } z;

	x.f = Position.X;
	y.f = Position.Y;
	z.f = Position.Z;

	return Murmur32( {
		Position.X == 0.0f ? 0u : x.i,
		Position.Y == 0.0f ? 0u : y.i,
		Position.Z == 0.0f ? 0u : z.i
	} );
}

FORCEINLINE uint32 Cycle3( uint32 Value )
{
	uint32 ValueMod3 = Value % 3;
	uint32 Value1Mod3 = ( 1 << ValueMod3 ) & 3;
	return Value - ValueMod3 + Value1Mod3;
}

FORCEINLINE uint32 Cycle3( uint32 Value, uint32 Offset )
{
	return Value - Value % 3 + ( Value + Offset ) % 3;
}


class FMeshSimplifier
{
public:
	QUADRICMESHREDUCTION_API		FMeshSimplifier( float* Verts, uint32 NumVerts, uint32* Indexes, uint32 NumIndexes, int32* MaterialIndexes, uint32 NumAttributes );
									~FMeshSimplifier() = default;

	void		SetAttributeWeights( const float* Weights )			{ AttributeWeights = Weights; }
	void		SetEdgeWeight( float Weight )						{ EdgeWeight = Weight; }
	void		SetMaxEdgeLengthFactor( float Factor )				{ MaxEdgeLengthFactor = Factor; }
	void		SetCorrectAttributes( void (*Function)( float* ) )	{ CorrectAttributes = Function; }
	void		SetLimitErrorToSurfaceArea( bool Value )			{ bLimitErrorToSurfaceArea = Value; }

	QUADRICMESHREDUCTION_API void	LockPosition( const FVector3f& Position );

	QUADRICMESHREDUCTION_API float	Simplify(
		uint32 TargetNumVerts, uint32 TargetNumTris, float TargetError,
		uint32 LimitNumVerts, uint32 LimitNumTris, float LimitError );
	QUADRICMESHREDUCTION_API void	PreserveSurfaceArea();
	QUADRICMESHREDUCTION_API void	ShrinkTriGroupWithMostSurfaceAreaLoss( float ShrinkAmount );
	QUADRICMESHREDUCTION_API void	ShrinkVoxelTriangles( float ShrinkAmount, TBitArray<>& InVoxelTriangles );
	QUADRICMESHREDUCTION_API void	DumpOBJ( const char* Filename );
	QUADRICMESHREDUCTION_API void	Compact();

	uint32		GetRemainingNumVerts() const	{ return RemainingNumVerts; }
	uint32		GetRemainingNumTris() const		{ return RemainingNumTris; }

	int32 DegreeLimit		= 24;
	float DegreePenalty		= 0.5f;
	float LockPenalty		= 1e8f;
	float InversionPenalty	= 100.0f;

protected:
	uint32		NumVerts;
	uint32		NumIndexes;
	uint32		NumAttributes;
	uint32		NumTris;

	uint32		RemainingNumVerts;
	uint32		RemainingNumTris;
	
	float*			Verts;
	uint32*			Indexes;
	int32*			MaterialIndexes;

	const float*	AttributeWeights = nullptr;
	float			EdgeWeight = 8.0f;
	float			MaxEdgeLengthFactor = 0.0f;
	void			(*CorrectAttributes)( float* ) = nullptr;
	bool			bLimitErrorToSurfaceArea = true;
	bool			bZeroWeights = false;

	FHashTable		VertHash;
	FHashTable		CornerHash;

	TArray< uint32 >	VertRefCount;
	TArray< uint8 >		CornerFlags;
	TBitArray<>			TriRemoved;

	struct FPerMaterialDeltas
	{
		float	SurfaceArea;
		int32	NumTris;
		int32	NumDisjoint;
	};
	TArray< FPerMaterialDeltas >	PerMaterialDeltas;

	struct FPair
	{
		FVector3f	Position0;
		FVector3f	Position1;
	};
	TArray< FPair >			Pairs;
	FHashTable				PairHash0;
	FHashTable				PairHash1;
	FBinaryHeap< float >	PairHeap;

	TArray< uint32 >	MovedVerts;
	TArray< uint32 >	MovedCorners;
	TArray< uint32 >	MovedPairs;
	TArray< uint32 >	ReevaluatePairs;

	TArray64< uint8 >		TriQuadrics;
	TArray< FEdgeQuadric >	EdgeQuadrics;
	TBitArray<>				EdgeQuadricsValid;

	TArray< float > WedgeAttributes;
	FDisjointSet	WedgeDisjointSet;
	
	TArray< float >	BackupVertData;
	int32			BackupMaterialIndex = INDEX_NONE;
	float			BackupArea			= 0.0f;
	float			TotalInputArea		= 0.0f;
	bool			bBackupValid		= false;

	enum ECornerFlags
	{
		MergeMask		= 3,		// Merge position 0 or 1
		AdjTriMask		= (1 << 2),	// Has been added to AdjTris
		LockedVertMask	= (1 << 3),	// Vert is locked, disallowing position movement
		RemoveTriMask	= (1 << 4),	// Triangle will overlap another after merge and should be removed
	};

protected:
	FVector3f&			GetPosition( uint32 VertIndex );
	const FVector3f&	GetPosition( uint32 VertIndex ) const;
	float*				GetAttributes( uint32 VertIndex );
	FQuadricAttr&		GetTriQuadric( uint32 TriIndex );

	template< typename FuncType >
	void	ForAllVerts( const FVector3f& Position, FuncType&& Function ) const;

	template< typename FuncType >
	void	ForAllCorners( const FVector3f& Position, FuncType&& Function ) const;

	template< typename FuncType >
	void	ForAllPairs( const FVector3f& Position, FuncType&& Function ) const;

	bool	AddUniquePair( FPair& Pair, uint32 PairIndex );

	void	CalcTriQuadric( uint32 TriIndex );
	void	CalcEdgeQuadric( uint32 EdgeIndex );

	float	EvaluateMerge( const FVector3f& Position0, const FVector3f& Position1, bool bMoveVerts );
	
	void	BeginMovePosition( const FVector3f& Position );
	void	EndMovePositions();

	uint32	CornerIndexMoved( uint32 TriIndex ) const;
	bool	TriWillInvert( uint32 TriIndex, const FVector3f& NewPosition ) const;

	void	RemoveTri( uint32 TriIndex );
	void	FixUpTri( uint32 TriIndex );
	bool	IsDuplicateTri( uint32 TriIndex ) const;
	void	SetVertIndex( uint32 Corner, uint32 NewVertIndex );
	void	RemoveDuplicateVerts( uint32 Corner );

	void	CaptureBackupTriangle( uint32 TriIndex, float Area );
	void	RestoreBackupTriangle( float& InOutMaxError );
};


FORCEINLINE FVector3f& FMeshSimplifier::GetPosition( uint32 VertIndex )
{
	return *reinterpret_cast< FVector3f* >( &Verts[ ( 3 + NumAttributes ) * VertIndex ] );
}

FORCEINLINE const FVector3f& FMeshSimplifier::GetPosition( uint32 VertIndex ) const
{
	return *reinterpret_cast< const FVector3f* >( &Verts[ ( 3 + NumAttributes ) * VertIndex ] );
}

FORCEINLINE float* FMeshSimplifier::GetAttributes( uint32 VertIndex )
{
	return &Verts[ ( 3 + NumAttributes ) * VertIndex + 3 ];
}

FORCEINLINE FQuadricAttr& FMeshSimplifier::GetTriQuadric( uint32 TriIndex )
{
	const SIZE_T QuadricSize = sizeof( FQuadricAttr ) + NumAttributes * 4 * sizeof( QScalar );
	return *reinterpret_cast< FQuadricAttr* >( &TriQuadrics[ TriIndex * QuadricSize ] );
}

template< typename FuncType >
void FMeshSimplifier::ForAllVerts( const FVector3f& Position, FuncType&& Function ) const
{
	uint32 Hash = HashPosition( Position );
	for( uint32 VertIndex = VertHash.First( Hash ); VertHash.IsValid( VertIndex ); VertIndex = VertHash.Next( VertIndex ) )
	{
		if( GetPosition( VertIndex ) == Position )
		{
			Function( VertIndex );
		}
	}
}

template< typename FuncType >
void FMeshSimplifier::ForAllCorners( const FVector3f& Position, FuncType&& Function ) const
{
	uint32 Hash = HashPosition( Position );
	for( uint32 Corner = CornerHash.First( Hash ); CornerHash.IsValid( Corner ); Corner = CornerHash.Next( Corner ) )
	{
		if( GetPosition( Indexes[ Corner ] ) == Position )
		{
			Function( Corner );
		}
	}
}

template< typename FuncType >
void FMeshSimplifier::ForAllPairs( const FVector3f& Position, FuncType&& Function ) const
{
	uint32 Hash = HashPosition( Position );
	for( uint32 PairIndex = PairHash0.First( Hash ); PairHash0.IsValid( PairIndex ); PairIndex = PairHash0.Next( PairIndex ) )
	{
		if( Pairs[ PairIndex ].Position0 == Position )
		{
			Function( PairIndex );
		}
	}

	for( uint32 PairIndex = PairHash1.First( Hash ); PairHash1.IsValid( PairIndex ); PairIndex = PairHash1.Next( PairIndex ) )
	{
		if( Pairs[ PairIndex ].Position1 == Position )
		{
			Function( PairIndex );
		}
	}
}