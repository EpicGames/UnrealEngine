// Copyright (C) 2009 Nine Realms, Inc
//

#pragma once

#include "CoreMinimal.h"
#include "UnrolledLinkList.h"

template< typename T > class TSimpTri;

enum ESimpElementFlags
{
	SIMP_DEFAULT	= 0,
	SIMP_REMOVED	= 1 << 0,
	SIMP_MARK1		= 1 << 1,
	SIMP_MARK2		= 1 << 2,
	SIMP_LOCKED		= 1 << 3,
};

template< typename T >
class TSimpTri;

template< typename T >
class TSimpVert
{
public:
					TSimpVert();

	FVector3f&		GetPos()					{ return vert.GetPos(); }
	const FVector3f&	GetPos() const				{ return vert.GetPos(); }
	float*			GetAttributes()				{ return vert.GetAttributes(); }
	const float*	GetAttributes() const		{ return vert.GetAttributes(); }

	// this vert
	void			EnableFlags( uint32 f );
	void			DisableFlags( uint32 f );
	bool			TestFlags( uint32 f ) const;

	void			EnableAdjVertFlags( uint32 f );
	void			DisableAdjVertFlags( uint32 f );

	void			EnableAdjTriFlags( uint32 f );
	void			DisableAdjTriFlags( uint32 f );

	template< typename Allocator >
	void			FindAdjacentVerts( TArray< TSimpVert<T>*, Allocator >& adjVerts );

	// all verts in group
	void			EnableFlagsGroup( uint32 f );
	void			DisableFlagsGroup( uint32 f );
	//bool			TestFlagsGroup( uint32 f ) const;

	void			EnableAdjVertFlagsGroup( uint32 f );
	void			DisableAdjVertFlagsGroup( uint32 f );

	void			EnableAdjTriFlagsGroup( uint32 f );
	void			DisableAdjTriFlagsGroup( uint32 f );

	template< typename Allocator >
	void			FindAdjacentVertsGroup( TArray< TSimpVert<T>*, Allocator >& adjVerts );

	typedef TUnrolledLinkList< TSimpTri<T>*, 8 >	TriList;
	typedef typename TriList::TIterator				TriIterator;
	
	// other verts sharing same point grouped
	TSimpVert<T>*	next;
	TSimpVert<T>*	prev;
	
	// bitfield of ESimpElementFlags
	uint32			flags;

	T				vert;

	// Adjacent triangles, all triangles which reference this vert
	// TODO could be link list of index buffer elements
	// /3 gives tri index
	TriList			adjTris;
};

template< typename T >
class TSimpTri
{
public:
	TSimpTri()
	: flags( SIMP_DEFAULT )
	{}

	void			EnableFlags( uint32 f );
	void			DisableFlags( uint32 f );
	bool			TestFlags( uint32 f ) const;
	
	bool			HasVertex( const TSimpVert<T>* v ) const;
	FVector3f			GetNormal() const;
	
	bool			ReplaceVertexIsValid( const TSimpVert<T>* oldV, const FVector3f& pos ) const;
	void			ReplaceVertex( TSimpVert<T>* oldV, TSimpVert<T>* newV );

	TSimpVert<T>*	verts[3];

	// bitfield of ESimpElementFlags
	uint32			flags;
};

template< typename T >
class TSimpEdge
{
public:
					TSimpEdge();

	void			EnableFlags( uint32 f );
	void			DisableFlags( uint32 f );
	bool			TestFlags( uint32 f ) const;

	// Edge group
	// Link list of all edges sharing the same end points, different attributes.
	// Need multiple edges to know which in both vert groups are connected.
	TSimpEdge*		next;
	TSimpEdge*		prev;

	TSimpVert<T>*	v0;
	TSimpVert<T>*	v1;

	// bitfield of ESimpElementFlags
	uint32			flags;
};


//=============
// TSimpVert
//=============
template< typename T >
FORCEINLINE TSimpVert<T>::TSimpVert()
: flags( SIMP_DEFAULT )
{
	next = this;
	prev = this;
}

template< typename T >
FORCEINLINE void TSimpVert<T>::EnableFlags( uint32 f )
{
	flags |= f;
}

template< typename T >
FORCEINLINE void TSimpVert<T>::DisableFlags( uint32 f )
{
	flags &= ~f;
}

template< typename T >
FORCEINLINE bool TSimpVert<T>::TestFlags( uint32 f ) const
{
	return ( flags & f ) == f;
}

template< typename T >
FORCEINLINE void TSimpVert<T>::EnableAdjVertFlags( uint32 f )
{
	for( TriIterator i = adjTris.Begin(); i != adjTris.End(); ++i )
	{
		for( int j = 0; j < 3; j++ )
		{
			(*i)->verts[j]->EnableFlags(f);
		}
	}
}

template< typename T >
FORCEINLINE void TSimpVert<T>::DisableAdjVertFlags( uint32 f )
{
	for( TriIterator i = adjTris.Begin(); i != adjTris.End(); ++i )
	{
		for( int j = 0; j < 3; j++ )
		{
			(*i)->verts[j]->DisableFlags(f);
		}
	}
}

template< typename T >
FORCEINLINE void TSimpVert<T>::EnableAdjTriFlags( uint32 f )
{
	for( TriIterator i = adjTris.Begin(); i != adjTris.End(); ++i )
	{
		(*i)->EnableFlags(f);
	}
}

template< typename T >
FORCEINLINE void TSimpVert<T>::DisableAdjTriFlags( uint32 f )
{
	for( TriIterator i = adjTris.Begin(); i != adjTris.End(); ++i )
	{
		(*i)->DisableFlags(f);
	}
}

template< typename T >
template< typename Allocator >
void TSimpVert<T>::FindAdjacentVerts( TArray< TSimpVert<T>*, Allocator >& adjVerts )
{
	// fill array
	for( TriIterator i = adjTris.Begin(); i != adjTris.End(); ++i )
	{
		for( int j = 0; j < 3; j++ )
		{
			TSimpVert* v = (*i)->verts[j];
			if( v != this )
			{
				adjVerts.AddUnique(v);
			}
		}
	}
}

template< typename T >
FORCEINLINE void TSimpVert<T>::EnableFlagsGroup( uint32 f )
{
	TSimpVert<T>* v = this;
	do {
		v->EnableFlags( f );
		v = v->next;
	} while( v != this );
}

template< typename T >
FORCEINLINE void TSimpVert<T>::DisableFlagsGroup( uint32 f )
{
	TSimpVert<T>* v = this;
	do {
		v->DisableFlags( f );
		v = v->next;
	} while( v != this );
}

template< typename T >
FORCEINLINE void TSimpVert<T>::EnableAdjVertFlagsGroup( uint32 f )
{
	TSimpVert<T>* v = this;
	do {
		v->EnableAdjVertFlags( f );
		v = v->next;
	} while( v != this );
}

template< typename T >
FORCEINLINE void TSimpVert<T>::DisableAdjVertFlagsGroup( uint32 f )
{
	TSimpVert<T>* v = this;
	do {
		v->DisableAdjVertFlags( f );
		v = v->next;
	} while( v != this );
}

template< typename T >
FORCEINLINE void TSimpVert<T>::EnableAdjTriFlagsGroup( uint32 f )
{
	TSimpVert<T>* v = this;
	do {
		v->EnableAdjTriFlags( f );
		v = v->next;
	} while( v != this );
}

template< typename T >
FORCEINLINE void TSimpVert<T>::DisableAdjTriFlagsGroup( uint32 f )
{
	TSimpVert<T>* v = this;
	do {
		v->DisableAdjTriFlags( f );
		v = v->next;
	} while( v != this );
}

template< typename T >
template< typename Allocator >
void TSimpVert<T>::FindAdjacentVertsGroup( TArray< TSimpVert<T>*, Allocator >& adjVerts )
{
	// fill array
	TSimpVert<T>* v = this;
	do {
		for( TriIterator i = adjTris.Begin(); i != adjTris.End(); ++i )
		{
			for( int j = 0; j < 3; j++ )
			{
				TSimpVert* TriVert = (*i)->verts[j];
				if( TriVert != v )
				{
					adjVerts.AddUnique( TriVert );
				}
			}
		}
		v = v->next;
	} while( v != this );
}


//=============
// TSimpTri
//=============
template< typename T >
FORCEINLINE void TSimpTri<T>::EnableFlags( uint32 f )
{
	flags |= f;
}

template< typename T >
FORCEINLINE void TSimpTri<T>::DisableFlags( uint32 f )
{
	flags &= ~f;
}

template< typename T >
FORCEINLINE bool TSimpTri<T>::TestFlags( uint32 f ) const
{
	return ( flags & f ) == f;
}

template< typename T >
FORCEINLINE bool TSimpTri<T>::HasVertex( const TSimpVert<T>* v ) const
{
	return ( v == verts[0] || v == verts[1] || v == verts[2] );
}

template< typename T >
FORCEINLINE FVector3f TSimpTri<T>::GetNormal() const
{
	FVector3f n = ( verts[2]->GetPos() - verts[0]->GetPos() ) ^ ( verts[1]->GetPos() - verts[0]->GetPos() );
	n.Normalize();
	return n;
}

template< typename T >
FORCEINLINE bool TSimpTri<T>::ReplaceVertexIsValid( const TSimpVert<T>* oldV, const FVector3f& pos ) const
{
	checkSlow( oldV );
	checkSlow( oldV == verts[0] || oldV == verts[1] || oldV == verts[2] );

	uint32 k;
	if( oldV == verts[0] )
		k = 0;
	else if( oldV == verts[1] )
		k = 1;
	else
		k = 2;
	
	const FVector3f& v0 = verts[k]->GetPos();
	const FVector3f& v1 = verts[ k = (1 << k) & 3 ]->GetPos();
	const FVector3f& v2 = verts[ k = (1 << k) & 3 ]->GetPos();
	
	const FVector3f d21 = v2 - v1;
	const FVector3f d01 = v0 - v1;
	const FVector3f dp1 = pos - v1;

	FVector3f n0 = d01 ^ d21;
	FVector3f n1 = dp1 ^ d21;

	return (n0 | n1) > 0.0f;
}

template< typename T >
FORCEINLINE void TSimpTri<T>::ReplaceVertex( TSimpVert<T>* oldV, TSimpVert<T>* newV )
{
	checkSlow( oldV && newV );
	checkSlow( oldV == verts[0] || oldV == verts[1] || oldV == verts[2] );
	checkSlow( newV != verts[0] && newV != verts[1] && newV != verts[2] );

	if( oldV == verts[0] )
		verts[0] = newV;
	else if( oldV == verts[1] )
		verts[1] = newV;
	else
		verts[2] = newV;

	checkSlow( !HasVertex( oldV ) );
}


//=============
// TSimpEdge
//=============
template< typename T >
FORCEINLINE TSimpEdge<T>::TSimpEdge()
: flags( SIMP_DEFAULT )
{
	next = this;
	prev = this;
}

template< typename T >
FORCEINLINE void TSimpEdge<T>::EnableFlags( uint32 f )
{
	flags |= f;
}

template< typename T >
FORCEINLINE void TSimpEdge<T>::DisableFlags( uint32 f )
{
	flags &= ~f;
}

template< typename T >
FORCEINLINE bool TSimpEdge<T>::TestFlags( uint32 f ) const
{
	return ( flags & f ) == f;
}
