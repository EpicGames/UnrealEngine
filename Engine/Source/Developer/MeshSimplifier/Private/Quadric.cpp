// Copyright (C) 2009 Nine Realms, Inc

#include "Quadric.h"
#include "MatrixUtil.h"

DEFINE_LOG_CATEGORY_STATIC( LogQuadric, Log, All );

#if defined(_MSC_VER) && !defined(__clang__)
#pragma float_control( precise, on, push )
#pragma warning(disable:6011)
#endif

FEdgeQuadric::FEdgeQuadric( const QVec3 p0, const QVec3 p1, const float Weight )
{
	n = p1 - p0;

	const QScalar Length = sqrt( n | n );
	if( Length < (QScalar)SMALL_NUMBER )
	{
		Zero();
		return;
	}
	else
	{
		n.x /= Length;
		n.y /= Length;
		n.z /= Length;
	}

	a = Weight * Length;
	
	nxx = a - a * n.x * n.x;
	nyy = a - a * n.y * n.y;
	nzz = a - a * n.z * n.z;

	nxy = -a * n.x * n.y;
	nxz = -a * n.x * n.z;
	nyz = -a * n.y * n.z;
}

FQuadric::FQuadric( const QVec3 p0, const QVec3 p1, const QVec3 p2 )
{
	const QVec3 p01 = p1 - p0;
	const QVec3 p02 = p2 - p0;

	// Compute the wedge product, giving the normal direction scaled by 
	// twice the triangle area.
	QVec3 n = p02 ^ p01;

	const QScalar Length = sqrt( n | n );
	const QScalar area = 0.5 * Length;
	if( Length < (QScalar)SMALL_NUMBER )
	{
		Zero();
		return;
	}
	else
	{
		n.x /= Length;
		n.y /= Length;
		n.z /= Length;
	}

	nxx = n.x * n.x;
	nyy = n.y * n.y;
	nzz = n.z * n.z;

	nxy = n.x * n.y;
	nxz = n.x * n.z;
	nyz = n.y * n.z;
	
	const QScalar dist = -( n | p0 );

	dn = dist * n;
	d2 = dist * dist;

#if WEIGHT_BY_AREA
	nxx *= area;
	nyy *= area;
	nzz *= area;

	nxy *= area;
	nxz *= area;
	nyz *= area;

	dn.x *= area;
	dn.y *= area;
	dn.z *= area;

	d2 *= area;

	a = area;
#else
	a = 1.0;
#endif
}

FQuadric::FQuadric( const QVec3 p )
{
	// (v - p)^T (v - p)
	// v^T I v - 2 p^T v + p^T p
	nxx = 1.0;
	nyy = 1.0;
	nzz = 1.0;

	nxy = 0.0;
	nxz = 0.0;
	nyz = 0.0;

	dn = -p;
	d2 = p | p;

	a = 0.0;
}

FQuadric::FQuadric( const QVec3 n, const QVec3 p )
{
	// nn^T = projection matrix
	//( v - nn^T v )^T ( v - nn^T v )
	// v^T ( I - nn^T ) v - 2p^T ( I - nn^T ) v + (p^T p - p^T nn^T p)
	nxx = 1.0 - n.x * n.x;
	nyy = 1.0 - n.y * n.y;
	nzz = 1.0 - n.z * n.z;

	nxy = -n.x * n.y;
	nxz = -n.x * n.z;
	nyz = -n.y * n.z;

	const QScalar dist = -( n | p );

	dn = -p - dist * n;
	d2 = (p | p) - dist * dist;

	a = 0.0;
}

float FQuadric::Evaluate( const FVector3f& Point ) const
{
	// Q(v) = vt*A*v + 2*bt*v + c
	
	// v = [ p ]
	//     [ s ]
	
	// A = [ C  B  ]
	//     [ Bt aI ]
	
	// C = n*nt
	// B = -g[ 0 .. m ]

	// b = [  dn         ]
	//     [ -d[ 0 .. m] ]

	// c = d2

	QVec3 p = Point;

	// A*v = [ C*p  + B*s ]
	//       [ Bt*p + a*s ]

	// C*p
	QScalar x = p | QVec3( nxx, nxy, nxz );
	QScalar y = p | QVec3( nxy, nyy, nyz );
	QScalar z = p | QVec3( nxz, nyz, nzz );

	// vt*A*v = pt * ( C*p + B*s ) + st * ( Bt*p + a*s )

	// pt * (C*p + B*s)
	QScalar vAv = p | QVec3( x, y, z );

	// bt*v
	QScalar btv = p | dn;

	// Q(v) = vt*A*v + 2*bt*v + c
	QScalar Q = vAv + 2.0 * btv + d2;
	
	if( Q < 0.0 || !FMath::IsFinite( Q ) )
	{
		Q = 0.0;
	}

	return Q;
}


FQuadricAttr::FQuadricAttr(
	const QVec3 p0, const QVec3 p1, const QVec3 p2,
	const float* attr0, const float* attr1, const float* attr2,
	const float* AttributeWeights, uint32 NumAttributes )
{
	const QVec3 p01 = p1 - p0;
	const QVec3 p02 = p2 - p0;

	// Compute the wedge product, giving the normal direction scaled by 
	// twice the triangle area.
	QVec3 n = p02 ^ p01;

#if VOLUME_CONSTRAINT
	// Already scaled by area*2
	nv = n;
	dv = -( n | p0 );
#endif

	const QScalar Length = sqrt( n | n );
	const QScalar area = 0.5 * Length;
	//if (Length < QScalar(SMALL_NUMBER))
	if( area < 1e-12 )
	{
		Zero( NumAttributes );
		return;
	}
	else
	{
		n.x /= Length;
		n.y /= Length;
		n.z /= Length;
	}
	
	nxx = n.x * n.x;
	nyy = n.y * n.y;
	nzz = n.z * n.z;

	nxy = n.x * n.y;
	nxz = n.x * n.z;
	nyz = n.y * n.z;
	
	const QScalar dist = -( n | p0 );

	dn = dist * n;
	d2 = dist * dist;


	// solve for g
	// (p1 - p0) | g = a1 - a0
	// (p2 - p0) | g = a2 - a0
	// n | g = 0
	QScalar A[] =
	{
		p01.x, p01.y, p01.z,
		p02.x, p02.y, p02.z,
		n.x,   n.y,   n.z
	};
	uint32 Pivot[3];
	bool bInvertable = LUPFactorize( A, Pivot, 3, (QScalar)1e-12 );

	QVec3* RESTRICT   g = (QVec3*)( this + 1 );
	QScalar* RESTRICT d = (QScalar*)( g + NumAttributes );
	
	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		if( AttributeWeights[i] == 0.0f )
		{
			g[i].x = 0.0;
			g[i].y = 0.0;
			g[i].z = 0.0;
			d[i] = 0.0;
			continue;
		}
		
		float a0 = AttributeWeights[i] * attr0[i];
		float a1 = AttributeWeights[i] * attr1[i];
		float a2 = AttributeWeights[i] * attr2[i];
		
		a0 = FMath::IsFinite( a0 ) ? a0 : 0.0f;
		a1 = FMath::IsFinite( a1 ) ? a1 : 0.0f;
		a2 = FMath::IsFinite( a2 ) ? a2 : 0.0f;

		QVec3 Grad;
		if( !bInvertable )
		{
			Grad.x = 0.0;
			Grad.y = 0.0;
			Grad.z = 0.0;
		}
		else
		{
			QScalar b[] =
			{
				a1 - a0,
				a2 - a0,
				0.0
			};
			LUPSolve( A, Pivot, 3, b, (QScalar*)&Grad );

			// Newton's method iterative refinement.
			{
				QScalar Residual[] =
				{
					b[0] - ( Grad | p01 ),
					b[1] - ( Grad | p02 ),
					b[2] - ( Grad | n )
				};
				TVec3< QScalar > Error;
				LUPSolve( A, Pivot, 3, Residual, (QScalar*)&Error );
				Grad = Grad + Error;
			}
		}

		g[i] = Grad;

		// p0 | g + d = a0
		d[i] = a0 - ( g[i] | p0 );

		nxx += g[i].x * g[i].x;
		nyy += g[i].y * g[i].y;
		nzz += g[i].z * g[i].z;

		nxy += g[i].x * g[i].y;
		nxz += g[i].x * g[i].z;
		nyz += g[i].y * g[i].z;

		dn += d[i] * g[i];
		d2 += d[i] * d[i];
	}

#if WEIGHT_BY_AREA
	nxx *= area;
	nyy *= area;
	nzz *= area;

	nxy *= area;
	nxz *= area;
	nyz *= area;

	dn.x *= area;
	dn.y *= area;
	dn.z *= area;

	d2 *= area;

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		g[i].x *= area;
		g[i].y *= area;
		g[i].z *= area;
		d[i] *= area;
	}

	a = area;
#else
	a = 1.0;
#endif
}

void FQuadricAttr::Rebase(
	const FVector3f& RESTRICT Point,
	const float* RESTRICT Attribute,
	const float* RESTRICT AttributeWeights,
	uint32 NumAttributes )
{
	//if( a < (QScalar)SMALL_NUMBER )
	if( a < 1e-12 )
		return;

	const QVec3 p0( Point );
	
	// Already scaled by area*2
	const QScalar InvA = 1.0 / a;
	const QScalar Dist2A = -( nv | p0 );
	const QScalar DistHalf = 0.25 * Dist2A * InvA;

	dn = DistHalf * nv;
	d2 = DistHalf * Dist2A;
	dv = Dist2A;

	QVec3* RESTRICT   g = (QVec3*)( this + 1 );
	QScalar* RESTRICT d = (QScalar*)( g + NumAttributes );

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		if( AttributeWeights[i] == 0.0f )
			continue;
		
		float a0 = AttributeWeights[i] * Attribute[i];

		checkSlow( FMath::IsFinite( a0 ) );

		// p0 | g + d = a0
		const QScalar qd = a0 - ( g[i] | p0 ) * InvA;

		d[i] = qd * a;
		dn  += qd * g[i];
		d2  += qd * d[i];
	}
}

void FQuadricAttr::Add(
	const FQuadricAttr& RESTRICT q,
	const FVector3f& RESTRICT Point,
	const float* RESTRICT Attribute,
	const float* RESTRICT AttributeWeights,
	uint32 NumAttributes )
{
	//if( q.a < (QScalar)SMALL_NUMBER )
	if( q.a < 1e-12 )
		return;

	nxx += q.nxx;
	nyy += q.nyy;
	nzz += q.nzz;

	nxy += q.nxy;
	nxz += q.nxz;
	nyz += q.nyz;
	
	const QVec3 p0( Point );

	// Already scaled by area*2
	const QScalar InvA = 1.0 / q.a;
	const QScalar Dist2A = -( q.nv | p0 );
	const QScalar DistHalf = 0.25 * Dist2A * InvA;

	dn += DistHalf * q.nv;
	d2 += DistHalf * Dist2A;

	nv += q.nv;
	dv += Dist2A;

	QVec3* RESTRICT   g = (QVec3*)( this + 1 );
	QScalar* RESTRICT d = (QScalar*)( g + NumAttributes );

	QVec3* RESTRICT   qg = (QVec3*)( &q + 1 );

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		if( AttributeWeights[i] == 0.0f )
			continue;
		
		float a0 = AttributeWeights[i] * Attribute[i];

		checkSlow( FMath::IsFinite( a0 ) );

		// p0 | g + d = a0
		const QScalar qd = a0 - ( qg[i] | p0 ) * InvA;
		const QScalar qda = qd * q.a;

		g[i] += qg[i];
		d[i] += qda;

		dn += qd * qg[i];
		d2 += qd * qda;
	}
	
	a += q.a;
}

void FQuadricAttr::Add(
	const FQuadricAttr& RESTRICT q,
	uint32 NumAttributes )
{
	nxx += q.nxx;
	nyy += q.nyy;
	nzz += q.nzz;

	nxy += q.nxy;
	nxz += q.nxz;
	nyz += q.nyz;

	dn += q.dn;
	d2 += q.d2;

	nv += q.nv;
	dv += q.dv;

	QVec3* RESTRICT   g = (QVec3*)( this + 1 );
	QScalar* RESTRICT d = (QScalar*)( g + NumAttributes );

	QVec3* RESTRICT   qg = (QVec3*)( &q + 1 );
	QScalar* RESTRICT qd = (QScalar*)( qg + NumAttributes );

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		g[i] += qg[i];
		d[i] += qd[i];
	}
	
	a += q.a;
}

void FQuadricAttr::Zero( uint32 NumAttributes )
{
	nxx = 0.0;
	nyy = 0.0;
	nzz = 0.0;

	nxy = 0.0;
	nxz = 0.0;
	nyz = 0.0;

	dn = 0.0;
	d2 = 0.0;

	QVec3* RESTRICT   g = (QVec3*)( this + 1 );
	QScalar* RESTRICT d = (QScalar*)( g + NumAttributes );

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		g[i] = 0.0;
		d[i] = 0.0;
	}
	
	a = 0.0;

#if VOLUME_CONSTRAINT
	nv = 0.0;
	dv = 0.0;
#endif
}

float FQuadricAttr::Evaluate( const FVector3f& RESTRICT Point, const float* RESTRICT Attributes, const float* RESTRICT AttributeWeights, uint32 NumAttributes ) const
{
	// Q(v) = vt*A*v + 2*bt*v + c
	
	// v = [ p ]
	//     [ s ]
	
	// A = [ C  B  ]
	//     [ Bt aI ]
	
	// C = n*nt
	// B = -g[ 0 .. m ]

	// b = [  dn         ]
	//     [ -d[ 0 .. m] ]

	// c = d2

	QVec3 p = Point;

	QVec3* RESTRICT   g = (QVec3*)( this + 1 );
	QScalar* RESTRICT d = (QScalar*)( g + NumAttributes );
	
	// A*v = [ C*p  + B*s ]
	//       [ Bt*p + a*s ]

	// C*p
	QScalar x = p | QVec3( nxx, nxy, nxz );
	QScalar y = p | QVec3( nxy, nyy, nyz );
	QScalar z = p | QVec3( nxz, nyz, nzz );

#if 0
	QScalar* RESTRICT s = (QScalar*)FMemory_Alloca( NumAttributes * sizeof( QScalar ) );

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		s[i] = AttributeWeights[i] * Attributes[i];
	}

	// B*s
	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		x -= g[i].x * s[i];
		y -= g[i].y * s[i];
		z -= g[i].z * s[i];
	}

	// vt*A*v = pt * ( C*p + B*s ) + st * ( Bt*p + a*s )

	// pt * (C*p + B*s)
	QScalar vAv = p | QVec3( x, y, z );

	// st * ( Bt*p + a*s )
	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		vAv += s[i] * ( a * s[i] - ( p | g[i] ) );
	}

	// bt*v
	QScalar btv = p | dn;
	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		btv -= d[i] * s[i];
	}

	// Q(v) = vt*A*v + 2*bt*v + c
	QScalar Q = vAv + 2.0 * btv + d2;
#else
	// Q(v) = vt*A*v + 2*bt*v + c
	QScalar Q = ( p | QVec3( x, y, z ) ) + 2.0 * ( p | dn ) + d2;

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		QScalar pgd = (p | g[i]) + d[i];
		QScalar s = AttributeWeights[i] * Attributes[i];

		// st * ( Bt*p + a*s + B + b )
		Q += s * ( a * s - 2.0 * pgd );
	}
#endif

	if( Q < 0.0 || !FMath::IsFinite( Q ) )
	{
		Q = 0.0;
	}

	return Q;
}

float FQuadricAttr::CalcAttributesAndEvaluate( const FVector3f& RESTRICT Point, float* RESTRICT Attributes, const float* RESTRICT AttributeWeights, uint32 NumAttributes ) const
{
	// Q(v) = vt*A*v + 2*bt*v + c
	
	// v = [ p ]
	//     [ s ]
	
	// A = [ C  B  ]
	//     [ Bt aI ]
	
	// C = n*nt
	// B = -g[ 0 .. m ]

	// b = [  dn         ]
	//     [ -d[ 0 .. m] ]

	// c = d2

	QVec3 p = Point;

	// A*v = [ C*p  + B*s ]
	//       [ Bt*p + a*s ]

#if 0
	// C*p + 2*bt*p
	QScalar x = ( p | QVec3( nxx, nxy, nxz ) ) + 2.0 * dn.x;
	QScalar y = ( p | QVec3( nxy, nyy, nyz ) ) + 2.0 * dn.y;
	QScalar z = ( p | QVec3( nxz, nyz, nzz ) ) + 2.0 * dn.z;
	QScalar w = 0.0;

	QVec3* RESTRICT   g = (QVec3*)( this + 1 );
	QScalar* RESTRICT d = (QScalar*)( g + NumAttributes );

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		if( AttributeWeights[i] != 0.0f )
		{
			QScalar s = ( (p | g[i]) + d[i] ) / a;
			Attributes[i] = s / AttributeWeights[i];

			// Many things cancel when s is the above.
			// s * ( a * s - g[i][0] * px - g[i][1] * py - g[i][2] * pz ) - 2.0*d[i]*s == -d[i] * s

			// B*s + b*s
			x -= g[i].x * s;
			y -= g[i].y * s;
			z -= g[i].z * s;
			w -= d[i]   * s;
		}
	}

	// vt*A*v = pt * ( C*p + B*s ) + st * ( Bt*p + a*s )
	QScalar vAv_2btv = ( p | QVec3( x, y, z ) ) + w;

	// Q(v) = vt*A*v + 2*bt*v + c
	QScalar Q = vAv_2btv + d2;
#else
	// C*p
	QScalar x = p | QVec3( nxx, nxy, nxz );
	QScalar y = p | QVec3( nxy, nyy, nyz );
	QScalar z = p | QVec3( nxz, nyz, nzz );

	// Q(v) = vt*A*v + 2*bt*v + c
	QScalar Q = ( p | QVec3( x, y, z ) ) + 2.0 * ( p | dn ) + d2;

	QVec3* RESTRICT   g = (QVec3*)( this + 1 );
	QScalar* RESTRICT d = (QScalar*)( g + NumAttributes );

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		if( AttributeWeights[i] != 0.0f )
		{
			QScalar pgd = (p | g[i]) + d[i];
			QScalar s = pgd / a;

			Attributes[i] = s / AttributeWeights[i];

			// Many things cancel when s is the above.
			// s * ( a * s - g[i][0] * px - g[i][1] * py - g[i][2] * pz ) - 2.0*d[i]*s == -d[i] * s

			// B*s + b*s
			Q -= pgd * s;
		}
	}
#endif

	if( Q < 0.0 || !FMath::IsFinite( Q ) )
	{
		Q = 0.0;
	}

	return Q;
}



bool FQuadricAttrOptimizer::Optimize( FVector3f& Position ) const
{
	// A * v = -b
	
	// v = [ p ]
	//     [ s ]
	
	// A = [ C  B  ]
	//     [ Bt aI ]
	
	// C = n*nt
	// B = -g[ 0 .. m ]

	// b = [  dn         ]
	//     [ -d[ 0 .. m] ]

	// ( C - 1/a * B*Bt ) * p = -1/a * B*d - dn
	if( a < 1e-12 )
	{
		return false;
	}
	QScalar InvA = 1.0 / a;
	
	// M = C - 1/a * B*Bt
	QScalar Mxx = nxx - BBtxx * InvA;
	QScalar Myy = nyy - BBtyy * InvA;
	QScalar Mzz = nzz - BBtzz * InvA;

	QScalar Mxy = nxy - BBtxy * InvA;
	QScalar Mxz = nxz - BBtxz * InvA;
	QScalar Myz = nyz - BBtyz * InvA;

	// -1/a * B*d - dn
	QVec3 aBddn = Bd * InvA - dn;

	/*
	float3x3 M =
	{
		Mxx, Mxy, Mxz,
		Mxy, Myy, Myz,
		Mxz, Myz, Mzz
	};
	float3 b = { aBddnx, aBddny, aBddnz };
	p = Inverse(M) * b;
	*/

	QScalar M[] =
	{
		Mxx, Mxy, Mxz,
		Mxy, Myy, Myz,
		Mxz, Myz, Mzz
	};
	QScalar b[] = { aBddn.x, aBddn.y, aBddn.z };

#if PSEUDO_INVERSE
	QScalar A[9];
	QScalar V[9];
	QScalar S[3];
	FMemory::Memcpy( A, M );

	JacobiSVD::EigenSolver3( A, S, V, (QScalar)SMALL_NUMBER );
	PseudoInverse( S, 3, 1e-6 );

	// Rebase
	for( int i = 0; i < 3; i++ )
		for( int j = 0; j < 3; j++ )
			b[i] -= M[ 3*i + j ] * Position[j];

	QScalar x[3];
	PseudoSolve( V, S, 3, b, x );
	//if( PseudoSolveIterate( M, V, S, 3, b, x ) )
	{
		Position.X += x[0];
		Position.Y += x[1];
		Position.Z += x[2];
		return true;
	}
#else
	uint32 Pivot[3];
	QScalar LU[9];
	FMemory::Memcpy( LU, M );
	if( LUPFactorize( LU, Pivot, 3, (QScalar)1e-12 ) )
	{
		QScalar p[3];
		if( LUPSolveIterate( M, LU, Pivot, 3, b, p ) )
		{
			Position.X = p[0];
			Position.Y = p[1];
			Position.Z = p[2];
			return true;
		}
	}
#endif

	return false;
}

bool FQuadricAttrOptimizer::OptimizeVolume( FVector3f& Position ) const
{
	// A * v = -b
	
	// v = [ p ]
	//     [ s ]
	
	// A = [ C  B  ]
	//     [ Bt aI ]
	
	// C = n*nt
	// B = -g[ 0 .. m ]

	// b = [  dn         ]
	//     [ -d[ 0 .. m] ]

	// ( C - 1/a * B*Bt ) * p = -1/a * B*d - dn
	if( a < 1e-12 )
	{
		return false;
	}
	QScalar InvA = 1.0 / a;
	
	// M = C - 1/a * B*Bt
	QScalar Mxx = nxx - BBtxx * InvA;
	QScalar Myy = nyy - BBtyy * InvA;
	QScalar Mzz = nzz - BBtzz * InvA;

	QScalar Mxy = nxy - BBtxy * InvA;
	QScalar Mxz = nxz - BBtxz * InvA;
	QScalar Myz = nyz - BBtyz * InvA;

	// -1/a * B*d - dn
	QVec3 aBddn = Bd * InvA - dn;

#if VOLUME_CONSTRAINT
	// Only use the volume constraint if it is well conditioned
	if( (nv | nv) > 1e-12 )
	{
		const QScalar M[] =
		{
			Mxx, Mxy, Mxz, nv.x,
			Mxy, Myy, Myz, nv.y,
			Mxz, Myz, Mzz, nv.z,
			nv.x, nv.y, nv.z, 0.0
		};
		QScalar b[] = { aBddn.x, aBddn.y, aBddn.z, -dv };

#if PSEUDO_INVERSE
		QScalar A[16];
		QScalar V[16];
		QScalar S[4];
		FMemory::Memcpy( A, M );

		JacobiSVD::EigenSolver4( A, S, V, (QScalar)SMALL_NUMBER );
		PseudoInverse( S, 4, 1e-6 );

		// Rebase
		for( int i = 0; i < 4; i++ )
			for( int j = 0; j < 3; j++ )
				b[i] -= M[ 4*i + j ] * Position[j];

		// Guess for the Lagrange multiplier
#if 1
		if( (nv | nv) > 1e-4 )
		{
			/*
			Guessing 0 for position (already rebased)
			M*0 + lm*nv = b
			nv * lm = b

			Solved with least squares (same as projection)
			A*x = b
			x = (A^T * A)^-1 * A^T * b
		
			lm = (nv^T * nv)^-1 * nv^T*b
			lm = (nv | b ) / (nv | nv);
			*/
			QScalar lm = ( nv.x * b[0] + nv.y * b[1] + nv.z * b[2] ) / ( nv | nv );
			// Rebase Lagrange multiplier
			for( int i = 0; i < 4; i++ )
				b[i] -= M[ 4*i + 3 ] * lm;
		}
#endif
		
		// Newton iterate Lagrange guess
		QScalar x[4];
		for( uint32 k = 0; k < 4; k++ )
		{
			PseudoSolve( V, S, 4, b, x );

			// Rebase Lagrange multiplier
			for( int i = 0; i < 4; i++ )
				b[i] -= M[ 4*i + 3 ] * x[3];
		}

		PseudoSolve( V, S, 4, b, x );
		//if( PseudoSolveIterate( M, V, S, 4, b, x ) )
		{
			Position.X += x[0];
			Position.Y += x[1];
			Position.Z += x[2];
			return true;
		}
#else
		uint32 Pivot[4];
		QScalar LU[16];
		FMemory::Memcpy( LU, M );
		if( LUPFactorize( LU, Pivot, 4, (QScalar)1e-12 ) )
		{
			QScalar p[4];
			if( LUPSolveIterate( M, LU, Pivot, 4, b, p ) )
			{
				Position.X = p[0];
				Position.Y = p[1];
				Position.Z = p[2];
				return true;
			}
		}
#endif
	}
#endif

	return false;
}

bool FQuadricAttrOptimizer::OptimizeLinear( const FVector3f& Position0, const FVector3f& Position1, FVector3f& Position ) const
{
	// Optimize on a line instead of full 3D.

	// A * v = -b
	
	// v = [ p ]
	//     [ s ]
	
	// A = [ C  B  ]
	//     [ Bt aI ]
	
	// C = n*nt
	// B = -g[ 0 .. m ]

	// b = [  dn         ]
	//     [ -d[ 0 .. m] ]

	// ( C - 1/a * B*Bt ) * p = -1/a * B*d - dn
	if( a < 1e-12 )
	{
		return false;
	}
	QScalar InvA = 1.0 / a;
	
	// M = C - 1/a * B*Bt
	QScalar Mxx = nxx - BBtxx * InvA;
	QScalar Myy = nyy - BBtyy * InvA;
	QScalar Mzz = nzz - BBtzz * InvA;

	QScalar Mxy = nxy - BBtxy * InvA;
	QScalar Mxz = nxz - BBtxz * InvA;
	QScalar Myz = nyz - BBtyz * InvA;

	// -1/a * B*d - dn
	QVec3 aBddn = Bd * InvA - dn;

	QVec3 p0( Position0 );
	QVec3 p1( Position1 );

	// M*p0
	QVec3 m0(
		p0.x * Mxx + p0.y * Mxy + p0.z * Mxz,
		p0.x * Mxy + p0.y * Myy + p0.z * Myz,
		p0.x * Mxz + p0.y * Myz + p0.z * Mzz
	);

	// M*p1
	QVec3 m1(
		p1.x * Mxx + p1.y * Mxy + p1.z * Mxz,
		p1.x * Mxy + p1.y * Myy + p1.z * Myz,
		p1.x * Mxz + p1.y * Myz + p1.z * Mzz
	);

	// M*p1 - M*p0
	QVec3 m01 = m1 - m0;

	/*
	float3x3 M =
	{
		Mxx, Mxy, Mxz,
		Mxy, Myy, Myz,
		Mxz, Myz, Mzz
	};
	float3 b = { aBddnx, aBddny, aBddnz };

	M * p = b
	M*( p0 + t*(p1 - p0) ) = b

	(M*p1 - M*p0) * t = b - M*p0
	m01 * t = b - m0

	Solved with least squares
	A*x = b
	x = (A^T * A)^-1 * A^T * b

	t = (m01^T * m01)^-1 * m01^T * (b - m0)
	t = ( m01 | (b - m0) ) / (m01 | m01)
	*/

	QScalar m01Sqr = m01 | m01;
	if( m01Sqr < 1e-16 )
	{
		return false;
	}

	QVec3 bm0 = aBddn - m0;

	QScalar t = (m01 | bm0) / m01Sqr;

#if VOLUME_CONSTRAINT
	QScalar nvSqr = nv | nv;

	// Only use the volume constraint if it is well conditioned
	if( nvSqr > 1e-12 )
	{
		/*
		*  If Volume Preservation is desired, a scalar Lagrange multiplier 'lm' is used to inflate the system
		*
		*      ( M,      nv )  ( p  )    = (  b  )
		*      ( nv^T,   0  )  ( lm )      ( -dv )
		*
		
			M * p + lm * nv = b
			nv^T * p = -dv

			M*( p0 + t*(p1 - p0) ) + lm*nv = b

			(M*p1 - M*p0) * t + nv * lm = b - M*p0
			(nv | p1 - nv | p0) * t = -dv - (nv | p0)

			[ M *  (p1 - p0),  nv ] [ t  ]  = [   b -  M * p0 ]
			[ nv | (p1 - p0),  0  ] [ lm ]    [ -dv - nv | p0 ]

			[ m01,  nv ] [ t  ]  = [   b - m0  ]
			[ nv01, 0  ] [ lm ]    [ -dv - nv0 ]

			Solved with least squares
			A*x = b
			x = (A^T * A)^-1 * A^T * b
		*/
		QScalar nv0  =  nv | p0;
		QScalar nv01 = (nv | p1) - nv0;

		// A^T * A =
		// [ m01 | m01 + nv01 | nv01,   m01 | nv ]
		// [ m01 | nv,                   nv | nv ]
		QScalar ATAxx = m01Sqr + nv01 * nv01;
		QScalar ATAxy = m01 | nv;
		QScalar ATAyy = nvSqr;

		QScalar det = ATAxx * ATAyy - ATAxy * ATAxy;

		if( FMath::Abs( det ) > 1e-16 )
		{
			// (A^T * A)^-1
			QScalar iATAxx =  ATAyy;
			QScalar iATAxy = -ATAxy;
			QScalar iATAyy =  ATAxx;

			// A^T * b
			// [ m01 | (b - m0) - (dv + nv0) * nv01 ]
			// [  nv | (b - m0)                     ]
			QScalar ATb[] =
			{
				(m01 | bm0) - (dv + nv0) * nv01,
				(nv  | bm0)
			};

			t = ( iATAxx * ATb[0] + iATAxy * ATb[1] ) / det;
		}
	}
#endif

	t = FMath::Clamp< QScalar >( t, 0.0, 1.0 );

	QVec3 p = p0 * (1.0 - t) + p1 * t;

	Position.X = p.x;
	Position.Y = p.y;
	Position.Z = p.z;

	return true;
}

#if defined(_MSC_VER) && !defined(__clang__)
#pragma float_control( pop )
#endif