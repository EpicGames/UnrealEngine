// Copyright (C) 2009 Nine Realms, Inc
//

#pragma once

#include "CoreMinimal.h"

// [ Hoppe 1999, "New Quadric Metric for Simplifying Meshes with Appearance Attributes" ]
// [ Hoppe 2000, "Efficient minimization of new quadric metric for simplifying meshes with appearance attributes" ]

// doubles needed for precision
#if defined(_MSC_VER) && !defined(__clang__)
#pragma float_control( precise, on, push )
#pragma warning(disable:6011)
#endif

#define WEIGHT_BY_AREA		1
#define VOLUME_CONSTRAINT	1
#define USE_FMA				0
#define PSEUDO_INVERSE		0

#if USE_FMA
// Kahan's algorithm
template< typename T >
inline T DifferenceOfProducts( T a, T b, T c, T d )
{
    T cd = c * d;
    T Error = std::fma( -c, d,  cd );
    T ab_cd = std::fma(  a, b, -cd );
    return ab_cd + Error;
}

// Knuth 1974
template< typename T >
T TwoSumError( T a, T b )
{
	T x = a + b;
	T z = x - b;
	return ( a - (x - z) ) + (b - z);
}

template< typename T >
T TwoProdError( T a, T b )
{
	T x = a * b;
	return std::fma( a, b, -x );
}
#endif

template< typename T >
struct TVec3
{
	T x;
	T y;
	T z;

	TVec3() = default;
	TVec3( T Scalar )
		: x( Scalar ), y( Scalar ), z( Scalar )
	{}

	TVec3( T InX, T InY, T InZ )
		: x( InX ), y( InY ), z( InZ )
	{}

	TVec3( const FVector3f& v )
		: x( v.X ), y( v.Y ), z( v.Z )
	{}

	TVec3 operator-() const
	{
		return TVec3<T>( -x, -y, -z );
	}

	TVec3 operator*( T Scalar ) const
	{
		return TVec3<T>(
			x * Scalar,
			y * Scalar,
			z * Scalar );
	}
	
	TVec3 operator+( const TVec3& v ) const
	{
		return TVec3<T>(
			x + v.x,
			y + v.y,
			z + v.z );
	}

	TVec3 operator-( const TVec3& v ) const
	{
		return TVec3<T>(
			x - v.x,
			y - v.y,
			z - v.z );
	}
	
	TVec3& operator+=( const TVec3& v )
	{
		x += v.x;
		y += v.y;
		z += v.z;
		return *this;
	}

	T operator|( const TVec3& v ) const
	{
#if USE_FMA
		/*
		T zz = z * v.z;
		T Error		= std::fma( z, v.z, -zz );
		T yy_zz		= std::fma( y, v.y, zz );
		T xx_yy_zz	= std::fma( x, v.z, yy_zz );
		return xx_yy_zz + Error;
		*/

		T xx = x * v.x;
		T yy = y * v.y;
		T zz = z * v.z;
		
		T Dot, DotError;
		DotError  = TwoProdError( x, v.x );
		Dot  = xx;
		DotError += TwoProdError( y, v.y ) + TwoSumError( Dot, yy );
		Dot += yy;
		DotError += TwoProdError( z, v.z ) + TwoSumError( Dot, zz );
		Dot += zz;

		return Dot + DotError;
#else
		return x * v.x + y * v.y + z * v.z;
#endif
	}

	TVec3 operator^( const TVec3& v ) const
	{
		TVec3 Result;
#if USE_FMA
		Result.x = DifferenceOfProducts( y, v.z, z, v.y );
		Result.y = DifferenceOfProducts( z, v.x, x, v.z );
		Result.z = DifferenceOfProducts( x, v.y, y, v.x );
#else
		Result.x = y * v.z - z * v.y;
		Result.y = z * v.x - x * v.z;
		Result.z = x * v.y - y * v.x;
#endif

		return Result;
	}
};

template< typename T >
FORCEINLINE TVec3<T> operator*( T Scalar, const TVec3<T>& v )
{
	return v.operator*( Scalar );
}

using QScalar = double;
using QVec3 = TVec3< QScalar >;


class FEdgeQuadric
{
public:
	FEdgeQuadric() {}
	FEdgeQuadric( const QVec3 p0, const QVec3 p1, const float Weight );
	FEdgeQuadric( const QVec3 p0, const QVec3 p1, const QVec3 FaceNormal, const float Weight );
	
	void		Zero();

	QScalar		nxx;
	QScalar		nyy;
	QScalar		nzz;

	QScalar		nxy;
	QScalar		nxz;
	QScalar		nyz;
	
	QVec3		n;

	QScalar		a;
};

inline FEdgeQuadric::FEdgeQuadric( const QVec3 p0, const QVec3 p1, const QVec3 FaceNormal, const float Weight )
{
	const QVec3 p01 = p1 - p0;

	n = p01 ^ FaceNormal;

	const QScalar Length = sqrt( n | n );
	if( Length < (QScalar)SMALL_NUMBER )
	{
		Zero();
		return;
	}
	else
	{
		n.x /= Length;
		n.y /= Length;
		n.z /= Length;
	}

	a = Weight * sqrt( p01 | p01 );
	
	nxx = a * n.x * n.x;
	nyy = a * n.y * n.y;
	nzz = a * n.z * n.z;

	nxy = a * n.x * n.y;
	nxz = a * n.x * n.z;
	nyz = a * n.y * n.z;
}

inline void FEdgeQuadric::Zero()
{
	nxx = 0.0;
	nyy = 0.0;
	nzz = 0.0;

	nxy = 0.0;
	nxz = 0.0;
	nyz = 0.0;

	n = 0.0;
	
	a = 0.0;
}


// Error quadric for position only
class FQuadric
{
public:
				FQuadric() {}
				// Distance from point
				FQuadric( const QVec3 p );
				// Distance from line, n must be normalized
				FQuadric( const QVec3 n, const QVec3 p );
				// Distance from triangle
				FQuadric( const QVec3 p0, const QVec3 p1, const QVec3 p2 );
	
	void		Zero();

	FQuadric&	operator+=( const FQuadric& q );

	void		Add( const FEdgeQuadric& q, const FVector3f& Point );
	
	// Evaluate error at point
	float		Evaluate( const FVector3f& p ) const;

	QScalar		nxx;
	QScalar		nyy;
	QScalar		nzz;

	QScalar		nxy;
	QScalar		nxz;
	QScalar		nyz;
	
	QVec3		dn;
	QScalar		d2;

	QScalar		a;
};

inline void FQuadric::Zero()
{
	nxx = 0.0;
	nyy = 0.0;
	nzz = 0.0;

	nxy = 0.0;
	nxz = 0.0;
	nyz = 0.0;

	dn = 0.0;
	d2 = 0.0;
	
	a = 0.0;
}

inline FQuadric& FQuadric::operator+=( const FQuadric& q )
{
	nxx += q.nxx;
	nyy += q.nyy;
	nzz += q.nzz;

	nxy += q.nxy;
	nxz += q.nxz;
	nyz += q.nyz;

	dn += q.dn;
	d2 += q.d2;
	
	a += q.a;

	return *this;
}

inline void FQuadric::Add( const FEdgeQuadric& RESTRICT EdgeQuadric, const FVector3f& Point )
{
	const QVec3 p0( Point );
	
	const QScalar Dist = -( EdgeQuadric.n | p0 );

	nxx += EdgeQuadric.nxx;
	nyy += EdgeQuadric.nyy;
	nzz += EdgeQuadric.nzz;

	nxy += EdgeQuadric.nxy;
	nxz += EdgeQuadric.nxz;
	nyz += EdgeQuadric.nyz;

	QScalar aDist = EdgeQuadric.a * Dist;
	//dn += aDist * EdgeQuadric.n;
	//d2 += aDist * Dist;

	dn += EdgeQuadric.a * -p0 - aDist * EdgeQuadric.n;
	d2 += EdgeQuadric.a * (p0 | p0) - aDist * Dist;
}


// Error quadric including attributes
// Attributes are assumed to be allocated immediately after this struct!
// See TQuadricAttr below for expected layout for a static case.
class FQuadricAttr : public FQuadric
{
public:
				FQuadricAttr() {}
				FQuadricAttr(
					const QVec3 p0, const QVec3 p1, const QVec3 p2,
					const float* a0, const float* a1, const float* a2,
					const float* AttributeWeights, uint32 NumAttributes
					);

	void		Rebase( const FVector3f& Point, const float* Attributes, const float* AttributeWeights, uint32 NumAttributes );
	void		Add( const FQuadricAttr& q, const FVector3f& Point, const float* Attribute, const float* AttributeWeights, uint32 NumAttributes );
	void		Add( const FQuadricAttr& q, uint32 NumAttributes );

	void		Zero( uint32 NumAttributes );
	
	// Evaluate error at point with attributes and weights
	float		Evaluate( const FVector3f& Point, const float* Attributes, const float* AttributeWeights, uint32 NumAttributes ) const;
	
	// Calculate attributes for point and evaluate error
	float		CalcAttributesAndEvaluate( const FVector3f& Point, float* Attributes, const float* AttributeWeights, uint32 NumAttributes ) const;

#if VOLUME_CONSTRAINT
	QVec3		nv;
	QScalar		dv;
#endif
};


// Error quadric including attributes
// Static NumAttributes version of FQuadricAttr.
template< uint32 NumAttributes >
class TQuadricAttr : public FQuadricAttr
{
public:
				TQuadricAttr() {}
				TQuadricAttr(
					const QVec3 p0, const QVec3 p1, const QVec3 p2,
					const float* a0, const float* a1, const float* a2,
					const float* AttributeWeights
					);

	void		Rebase( const FVector3f& Point, const float* Attributes, const float* AttributeWeights );
	void		Add( const TQuadricAttr< NumAttributes >& q, const FVector3f& Point, const float* Attribute, const float* AttributeWeights );

	void		Zero();
	
	TQuadricAttr< NumAttributes >& operator+=( const FQuadric& q );
	TQuadricAttr< NumAttributes >& operator+=( const TQuadricAttr< NumAttributes >& q );
	
	// Evaluate error at point with attributes and weights
	float		Evaluate( const FVector3f& Point, const float* Attributes, const float* AttributeWeights ) const;
	
	// Calculate attributes for point and evaluate error
	float		CalcAttributesAndEvaluate( const FVector3f& Point, float* Attributes, const float* AttributeWeights ) const;
	
	QScalar		g[ NumAttributes ][3];
	QScalar		d[ NumAttributes ];
};

template< uint32 NumAttributes >
inline TQuadricAttr< NumAttributes >::TQuadricAttr(
	const QVec3 p0, const QVec3 p1, const QVec3 p2,
	const float* a0, const float* a1, const float* a2,
	const float* AttributeWeights )
	: FQuadricAttr(
		p0, p1, p2,
		a0, a1, a2,
		AttributeWeights,
		NumAttributes )
{}

template< uint32 NumAttributes >
inline void TQuadricAttr< NumAttributes >::Rebase(
	const FVector3f& Point, const float* Attribute,
	const float* AttributeWeights )
{
	Rebase( Point, Attribute, AttributeWeights, NumAttributes );
}

template< uint32 NumAttributes >
inline void TQuadricAttr< NumAttributes >::Zero()
{
	Zero( NumAttributes );
}

template< uint32 NumAttributes >
inline TQuadricAttr< NumAttributes >& TQuadricAttr< NumAttributes >::operator+=( const TQuadricAttr< NumAttributes >& RESTRICT q )
{
	nxx += q.nxx;
	nyy += q.nyy;
	nzz += q.nzz;

	nxy += q.nxy;
	nxz += q.nxz;
	nyz += q.nyz;

	dn += q.dn;
	d2 += q.d2;

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		g[i][0] += q.g[i][0];
		g[i][1] += q.g[i][1];
		g[i][2] += q.g[i][2];
		d[i] += q.d[i];
	}
	
	a += q.a;

#if VOLUME_CONSTRAINT
	nv += q.nv;
	dv += q.dv;
#endif

	return *this;
}

template< uint32 NumAttributes >
inline float TQuadricAttr< NumAttributes >::CalcAttributesAndEvaluate( const FVector3f& Point, float* RESTRICT Attributes, const float* RESTRICT AttributeWeights ) const
{
	return FQuadricAttr::CalcAttributesAndEvaluate( Point, Attributes, AttributeWeights, NumAttributes );
}


class FQuadricAttrOptimizer
{
public:
				FQuadricAttrOptimizer();

	void		AddQuadric( const FQuadric& q );
	void		AddQuadric( const FQuadricAttr& q, uint32 NumAttributes );

	// Find optimal point for minimal error
	bool		Optimize( FVector3f& Position ) const;
	bool		OptimizeVolume( FVector3f& Position ) const;
	bool		OptimizeLinear( const FVector3f& Position0, const FVector3f& Position1, FVector3f& Position ) const;

private:
	QScalar		nxx;
	QScalar		nyy;
	QScalar		nzz;

	QScalar		nxy;
	QScalar		nxz;
	QScalar		nyz;
	
	QVec3		dn;

	QScalar		a;

#if VOLUME_CONSTRAINT
	QVec3		nv;
	QScalar		dv;
#endif

	QScalar		BBtxx;
	QScalar		BBtyy;
	QScalar		BBtzz;
	QScalar		BBtxy;
	QScalar		BBtxz;
	QScalar		BBtyz;

	QVec3		Bd;
};

inline FQuadricAttrOptimizer::FQuadricAttrOptimizer()
{
	nxx = 0.0;
	nyy = 0.0;
	nzz = 0.0;

	nxy = 0.0;
	nxz = 0.0;
	nyz = 0.0;

	dn = 0.0;
	
	a = 0.0;

#if VOLUME_CONSTRAINT
	nv = 0.0;
	dv = 0.0;
#endif

	BBtxx = 0.0;
	BBtyy = 0.0;
	BBtzz = 0.0;
	BBtxy = 0.0;
	BBtxz = 0.0;
	BBtyz = 0.0;

	Bd = 0.0;
}

inline void FQuadricAttrOptimizer::AddQuadric( const FQuadric& RESTRICT q )
{
	nxx += q.nxx;
	nyy += q.nyy;
	nzz += q.nzz;

	nxy += q.nxy;
	nxz += q.nxz;
	nyz += q.nyz;

	dn += q.dn;
	
	//a += q.a;
}

inline void FQuadricAttrOptimizer::AddQuadric( const FQuadricAttr& RESTRICT q, uint32 NumAttributes )
{
	if( q.a < (QScalar)SMALL_NUMBER )
		return;

	nxx += q.nxx;
	nyy += q.nyy;
	nzz += q.nzz;

	nxy += q.nxy;
	nxz += q.nxz;
	nyz += q.nyz;

	dn += q.dn;
	
	a += q.a;

#if VOLUME_CONSTRAINT
	nv += q.nv;
	dv += q.dv;
#endif

	QVec3* RESTRICT   g = (QVec3*)( &q + 1 );
	QScalar* RESTRICT d = (QScalar*)( g + NumAttributes );

	for( uint32 i = 0; i < NumAttributes; i++ )
	{
		// B*Bt
		BBtxx += g[i].x * g[i].x;
		BBtyy += g[i].y * g[i].y;
		BBtzz += g[i].z * g[i].z;

		BBtxy += g[i].x * g[i].y;
		BBtxz += g[i].x * g[i].z;
		BBtyz += g[i].y * g[i].z;
		
		// -B*d
		Bd += g[i] * d[i];
	}
}

#if defined(_MSC_VER) && !defined(__clang__)
#pragma float_control( pop )
#endif
