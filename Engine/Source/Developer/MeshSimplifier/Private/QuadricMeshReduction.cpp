// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "MeshBuild.h"
#include "MeshSimplify.h"
#include "LerpVert.h"
#include "OverlappingCorners.h"
#include "Templates/UniquePtr.h"
#include "Features/IModularFeatures.h"
#include "IMeshReductionInterfaces.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "RenderUtils.h"
#include "Engine/StaticMesh.h"
#include "Misc/ConfigCacheIni.h"
#include "RenderMath.h"

class FQuadricSimplifierMeshReductionModule : public IMeshReductionModule
{
public:
	virtual ~FQuadricSimplifierMeshReductionModule() {}

	// IModuleInterface interface.
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// IMeshReductionModule interface.
	virtual class IMeshReduction* GetStaticMeshReductionInterface() override;
	virtual class IMeshReduction* GetSkeletalMeshReductionInterface() override;
	virtual class IMeshMerging* GetMeshMergingInterface() override;
	virtual class IMeshMerging* GetDistributedMeshMergingInterface() override;	
	virtual FString GetName() override;
};


DEFINE_LOG_CATEGORY_STATIC(LogQuadricSimplifier, Log, All);
IMPLEMENT_MODULE(FQuadricSimplifierMeshReductionModule, QuadricMeshReduction);

void CorrectAttributes( float* Attributes )
{
	FVector3f& TangentX	= *reinterpret_cast< FVector3f* >( Attributes );
	FVector3f& TangentY	= *reinterpret_cast< FVector3f* >( Attributes + 3 );
	FVector3f& TangentZ	= *reinterpret_cast< FVector3f* >( Attributes + 3 + 3 );
	FLinearColor& Color	= *reinterpret_cast< FLinearColor* >( Attributes + 3 + 3 + 3 );

	TangentZ.Normalize();
	TangentX -= ( TangentX | TangentZ ) * TangentZ;
	TangentX.Normalize();
	TangentY -= ( TangentY | TangentZ ) * TangentZ;
	TangentY -= ( TangentY | TangentX ) * TangentX;
	TangentY.Normalize();
	Color = Color.GetClamped();
}

bool VertsEqual( const FLerpVert& A, const FLerpVert& B )
{
	if( !PointsEqual(	A.Position, B.Position ) ||
		!NormalsEqual(	A.TangentX, B.TangentX ) ||
		!NormalsEqual(	A.TangentY, B.TangentY ) ||
		!NormalsEqual(	A.TangentZ, B.TangentZ ) ||
		!A.Color.Equals( B.Color ) )
	{
		return false;
	}

	// UVs
	for( int32 UVIndex = 0; UVIndex < MAX_STATIC_TEXCOORDS; UVIndex++ )
	{
		if( !UVsEqual( A.UVs[ UVIndex ], B.UVs[ UVIndex ] ) )
		{
			return false;
		}
	}

	return true;
}

class FQuadricSimplifierMeshReduction : public IMeshReduction
{
public:
	virtual const FString& GetVersionString() const override
	{
		// Correct layout selection depends on the name "QuadricMeshReduction_{foo}"
		// e.g.
		// TArray<FString> SplitVersionString;
		// VersionString.ParseIntoArray(SplitVersionString, TEXT("_"), true);
		// bool bUseQuadricSimplier = SplitVersionString[0].Equals("QuadricMeshReduction");

		static FString Version = TEXT("QuadricMeshReduction_V2.1");
		return Version;
	}

	virtual void ReduceMeshDescription(
		FMeshDescription& OutReducedMesh,
		float& OutMaxDeviation,
		const FMeshDescription& InMesh,
		const FOverlappingCorners& InOverlappingCorners,
		const struct FMeshReductionSettings& ReductionSettings
	) override
	{
		check(&InMesh != &OutReducedMesh);	// can't reduce in-place
		TRACE_CPUPROFILER_EVENT_SCOPE(FQuadricSimplifierMeshReduction::ReduceMeshDescription);
		const uint32 NumTexCoords = MAX_STATIC_TEXCOORDS;
		int32 InMeshNumTexCoords = 1;
		
		TMap<FVertexID, FVertexID> VertexIDRemap;

		bool bWeldVertices = ReductionSettings.WeldingThreshold > 0.0f;
		if (bWeldVertices)
		{
			FStaticMeshOperations::BuildWeldedVertexIDRemap(InMesh, ReductionSettings.WeldingThreshold, VertexIDRemap);
		}

		TArray< FLerpVert >	Verts;
		TArray< uint32 >	Indexes;
		TArray< int32 >		MaterialIndexes;

		TMap< int32, int32 > VertsMap;

		int32 NumFaces = InMesh.Triangles().Num();
		int32 NumWedges = NumFaces * 3;
		const FStaticMeshConstAttributes InMeshAttribute(InMesh);
		TVertexAttributesConstRef<FVector3f> InVertexPositions = InMeshAttribute.GetVertexPositions();
		TVertexInstanceAttributesConstRef<FVector3f> InVertexNormals = InMeshAttribute.GetVertexInstanceNormals();
		TVertexInstanceAttributesConstRef<FVector3f> InVertexTangents = InMeshAttribute.GetVertexInstanceTangents();
		TVertexInstanceAttributesConstRef<float> InVertexBinormalSigns = InMeshAttribute.GetVertexInstanceBinormalSigns();
		TVertexInstanceAttributesConstRef<FVector4f> InVertexColors = InMeshAttribute.GetVertexInstanceColors();
		TVertexInstanceAttributesConstRef<FVector2f> InVertexUVs = InMeshAttribute.GetVertexInstanceUVs();
		TPolygonGroupAttributesConstRef<FName> InPolygonGroupMaterialNames = InMeshAttribute.GetPolygonGroupMaterialSlotNames();

		TPolygonGroupAttributesRef<FName> OutPolygonGroupMaterialNames = OutReducedMesh.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName);

		float SurfaceArea = 0.0f;

		int32 WedgeIndex = 0;
		for (const FTriangleID TriangleID : InMesh.Triangles().GetElementIDs())
		{
			const FPolygonGroupID PolygonGroupID = InMesh.GetTrianglePolygonGroup(TriangleID);
			TArrayView<const FVertexID> VertexIDs = InMesh.GetTriangleVertices(TriangleID);

			FVector3f CornerPositions[3];
			for (int32 TriVert = 0; TriVert < 3; ++TriVert)
			{
				const FVertexID TmpVertexID = VertexIDs[TriVert];
				const FVertexID VertexID = bWeldVertices ? VertexIDRemap[TmpVertexID] : TmpVertexID;
				CornerPositions[TriVert] = InVertexPositions[VertexID];
			}

			// Don't process degenerate triangles.
			if( PointsEqual(CornerPositions[0], CornerPositions[1]) ||
				PointsEqual(CornerPositions[0], CornerPositions[2]) ||
				PointsEqual(CornerPositions[1], CornerPositions[2]) )
			{
				WedgeIndex += 3;
				continue;
			}

			int32 VertexIndices[3];
			for (int32 TriVert = 0; TriVert < 3; ++TriVert, ++WedgeIndex)
			{
				const FVertexInstanceID VertexInstanceID = InMesh.GetTriangleVertexInstance(TriangleID, TriVert);
				const int32 VertexInstanceValue = VertexInstanceID.GetValue();
				const FVector3f& VertexPosition = CornerPositions[TriVert];

				FLerpVert NewVert;

				NewVert.Position = CornerPositions[TriVert];
				NewVert.TangentX = InVertexTangents[ VertexInstanceID ];
				NewVert.TangentZ = InVertexNormals[ VertexInstanceID ];
				NewVert.TangentY = FVector3f(0.0f);
				if (!NewVert.TangentZ.IsNearlyZero(SMALL_NUMBER) && !NewVert.TangentX.IsNearlyZero(SMALL_NUMBER))
				{
					NewVert.TangentY = FVector3f::CrossProduct(NewVert.TangentZ, NewVert.TangentX).GetSafeNormal() * InVertexBinormalSigns[ VertexInstanceID ];
				}

				// Fix bad tangents
				NewVert.TangentX = NewVert.TangentX.ContainsNaN() ? FVector3f::ZeroVector : NewVert.TangentX;
				NewVert.TangentY = NewVert.TangentY.ContainsNaN() ? FVector3f::ZeroVector : NewVert.TangentY;
				NewVert.TangentZ = NewVert.TangentZ.ContainsNaN() ? FVector3f::ZeroVector : NewVert.TangentZ;
				NewVert.Color = FLinearColor(InVertexColors[ VertexInstanceID ]);

				for (int32 UVIndex = 0; UVIndex < NumTexCoords; UVIndex++)
				{
					if (UVIndex < InVertexUVs.GetNumChannels())
					{
						NewVert.UVs[UVIndex] = InVertexUVs.Get(VertexInstanceID, UVIndex);
						InMeshNumTexCoords = FMath::Max(UVIndex + 1, InMeshNumTexCoords);
					}
					else
					{
						NewVert.UVs[UVIndex] = FVector2f::ZeroVector;
					}
				}

				// Make sure this vertex is valid from the start
				CorrectAttributes( (float*)&NewVert.TangentX );
					

				//Never add duplicated vertex instance
				//Use WedgeIndex since OverlappingCorners has been built based on that
				const TArray<int32>& DupVerts = InOverlappingCorners.FindIfOverlapping(WedgeIndex);

				int32 Index = INDEX_NONE;
				for (int32 k = 0; k < DupVerts.Num(); k++)
				{
					if (DupVerts[k] >= WedgeIndex)
					{
						// the verts beyond me haven't been placed yet, so these duplicates are not relevant
						break;
					}

					int32* Location = VertsMap.Find(DupVerts[k]);
					if (Location)
					{
						FLerpVert& FoundVert = Verts[*Location];
						if( VertsEqual( NewVert, FoundVert ) )
						{
							Index = *Location;
							break;
						}
					}
				}
				if (Index == INDEX_NONE)
				{
					Index = Verts.Add(NewVert);
					VertsMap.Add(WedgeIndex, Index);
				}
				VertexIndices[TriVert] = Index;
			}
				
			// Reject degenerate triangles.
			if (VertexIndices[0] == VertexIndices[1] ||
				VertexIndices[1] == VertexIndices[2] ||
				VertexIndices[0] == VertexIndices[2])
			{
				continue;
			}

			{
				FVector3f Edge01 = CornerPositions[1] - CornerPositions[0];
				FVector3f Edge12 = CornerPositions[2] - CornerPositions[1];
				FVector3f Edge20 = CornerPositions[0] - CornerPositions[2];

				float TriArea = 0.5f * ( Edge01 ^ Edge20 ).Size();
				SurfaceArea += TriArea;
			}

			Indexes.Add(VertexIndices[0]);
			Indexes.Add(VertexIndices[1]);
			Indexes.Add(VertexIndices[2]);

			MaterialIndexes.Add( PolygonGroupID.GetValue() );
		}

		uint32 NumVerts = Verts.Num();
		uint32 NumIndexes = Indexes.Num();
		uint32 NumTris = NumIndexes / 3;

		//Get the targets tris and verts from the reduction criterion
		uint32 TargetNumTris = NumTris;;
		uint32 TargetNumVerts = NumVerts;
		if (ReductionSettings.TerminationCriterion == EStaticMeshReductionTerimationCriterion::Triangles
			|| ReductionSettings.TerminationCriterion == EStaticMeshReductionTerimationCriterion::Any)
		{
			TargetNumTris = FMath::CeilToInt(NumTris * ReductionSettings.PercentTriangles);
			TargetNumTris = FMath::Min(ReductionSettings.MaxNumOfTriangles, TargetNumTris);
		}

		if (ReductionSettings.TerminationCriterion == EStaticMeshReductionTerimationCriterion::Vertices
			|| ReductionSettings.TerminationCriterion == EStaticMeshReductionTerimationCriterion::Any)
		{
			TargetNumVerts = FMath::CeilToInt(NumVerts * ReductionSettings.PercentVertices);
			TargetNumVerts = FMath::Min(ReductionSettings.MaxNumOfVerts, TargetNumVerts);
		}

		// Keep a minimum of 64 triangles, to prevent already low-poly meshes from losing too much of their silhouette.
		TargetNumTris = FMath::Max(TargetNumTris, 64u);
		// Clamp to a minimum of 4 vertices, ReductionSettings.PercentVertices can be zero which makes TargetNumVerts also zero
		TargetNumVerts = FMath::Max(TargetNumVerts, 4u);

		if( TargetNumVerts < NumVerts || TargetNumTris < NumTris )
		{
			const uint32 NumAttributes = ( sizeof( FLerpVert ) - sizeof( FVector3f ) ) / sizeof(float);
			float AttributeWeights[ NumAttributes ] =
			{
				0.1f, 0.1f, 0.1f,	// Tangent[0]
				0.1f, 0.1f, 0.1f,	// Tangent[1]
				16.0f, 16.0f, 16.0f	// Normal
			};
			float* ColorWeights = AttributeWeights + 9;
			float* UVWeights = ColorWeights + 4;

			bool bHasColors = true;

			// Set weights if they are used
			if( bHasColors )
			{
				ColorWeights[0] = 0.1f;
				ColorWeights[1] = 0.1f;
				ColorWeights[2] = 0.1f;
				ColorWeights[3] = 0.1f;
			}

			float UVWeight = 0.5f;
			for( int32 UVIndex = 0; UVIndex < InVertexUVs.GetNumChannels(); UVIndex++ )
			{
				// Normalize UVWeights using min/max UV range.

				float MinUV = +FLT_MAX;
				float MaxUV = -FLT_MAX;

				for( int32 VertexIndex = 0; VertexIndex < Verts.Num(); VertexIndex++ )
				{
					MinUV = FMath::Min( MinUV, Verts[ VertexIndex ].UVs[ UVIndex ].X );
					MinUV = FMath::Min( MinUV, Verts[ VertexIndex ].UVs[ UVIndex ].Y );
					MaxUV = FMath::Max( MaxUV, Verts[ VertexIndex ].UVs[ UVIndex ].X );
					MaxUV = FMath::Max( MaxUV, Verts[ VertexIndex ].UVs[ UVIndex ].Y );
				}

				UVWeights[ 2 * UVIndex + 0 ] = UVWeight / FMath::Max( 1.0f, MaxUV - MinUV );
				UVWeights[ 2 * UVIndex + 1 ] = UVWeight / FMath::Max( 1.0f, MaxUV - MinUV );
			}

			FMeshSimplifier Simplifier( (float*)Verts.GetData(), Verts.Num(), Indexes.GetData(), Indexes.Num(), MaterialIndexes.GetData(), NumAttributes );

			Simplifier.SetAttributeWeights( AttributeWeights );
			Simplifier.SetCorrectAttributes( CorrectAttributes );
			Simplifier.SetEdgeWeight( 512.0f );
			Simplifier.SetLimitErrorToSurfaceArea( false );

			Simplifier.DegreePenalty = 100.0f;
			Simplifier.InversionPenalty = 1000000.0f;

			float MaxErrorSqr = Simplifier.Simplify( TargetNumVerts, TargetNumTris, 0.0f, 4, 2, MAX_flt );

			if( Simplifier.GetRemainingNumVerts() == 0 || Simplifier.GetRemainingNumTris() == 0 )
			{
				// Reduced to nothing so just return the orignial.
				OutReducedMesh = InMesh;
				OutMaxDeviation = 0.0f;
				return;
			}
	
			Simplifier.Compact();

			Verts.SetNum( Simplifier.GetRemainingNumVerts() );
			Indexes.SetNum( Simplifier.GetRemainingNumTris() * 3 );
			MaterialIndexes.SetNum( Simplifier.GetRemainingNumTris() );

			NumVerts = Simplifier.GetRemainingNumVerts();
			NumTris = Simplifier.GetRemainingNumTris();
			NumIndexes = NumTris * 3;

			OutMaxDeviation = FMath::Sqrt( MaxErrorSqr ) / 8.0f;
		}
		else
		{
			// Rare but could happen with rounding or only 2 triangles.
			OutMaxDeviation = 0.0f;
		}

		{
			//Empty the destination mesh
			OutReducedMesh.Empty();

			//Fill the PolygonGroups from the InMesh
			for (const FPolygonGroupID PolygonGroupID : InMesh.PolygonGroups().GetElementIDs())
			{
				OutReducedMesh.CreatePolygonGroupWithID(PolygonGroupID);
				OutPolygonGroupMaterialNames[PolygonGroupID] = InPolygonGroupMaterialNames[PolygonGroupID];
			}

			TVertexAttributesRef<FVector3f> OutVertexPositions = OutReducedMesh.GetVertexPositions();

			//Fill the vertex array
			for (int32 VertexIndex = 0; VertexIndex < (int32)NumVerts; ++VertexIndex)
			{
				FVertexID AddedVertexId = OutReducedMesh.CreateVertex();
				OutVertexPositions[AddedVertexId] = Verts[VertexIndex].Position;
				check(AddedVertexId.GetValue() == VertexIndex);
			}

			TMap<int32, FPolygonGroupID> PolygonGroupMapping;

			FStaticMeshAttributes Attributes(OutReducedMesh);
			TVertexInstanceAttributesRef<FVector3f> OutVertexNormals = Attributes.GetVertexInstanceNormals();
			TVertexInstanceAttributesRef<FVector3f> OutVertexTangents = Attributes.GetVertexInstanceTangents();
			TVertexInstanceAttributesRef<float> OutVertexBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
			TVertexInstanceAttributesRef<FVector4f> OutVertexColors = Attributes.GetVertexInstanceColors();
			TVertexInstanceAttributesRef<FVector2f> OutVertexUVs = Attributes.GetVertexInstanceUVs();

			//Specify the number of texture coords in this mesh description
			OutVertexUVs.SetNumChannels(InMeshNumTexCoords);

			//Vertex instances and Polygons
			for (int32 TriangleIndex = 0; TriangleIndex < (int32)NumTris; TriangleIndex++)
			{
				FVertexInstanceID CornerInstanceIDs[3];

				FVertexID CornerVerticesIDs[3];
				for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
				{
					int32 VertexInstanceIndex = TriangleIndex * 3 + CornerIndex;
					const FVertexInstanceID VertexInstanceID(VertexInstanceIndex);
					CornerInstanceIDs[CornerIndex] = VertexInstanceID;
					int32 ControlPointIndex = Indexes[VertexInstanceIndex];
					const FVertexID VertexID(ControlPointIndex);
					//FVector3f VertexPosition = OutReducedMesh.GetVertex(VertexID).VertexPosition;
					CornerVerticesIDs[CornerIndex] = VertexID;
					FVertexInstanceID AddedVertexInstanceId = OutReducedMesh.CreateVertexInstance(VertexID);
					//Make sure the Added vertex instance ID is matching the expected vertex instance ID
					check(AddedVertexInstanceId == VertexInstanceID);
					check(AddedVertexInstanceId.GetValue() == VertexInstanceIndex);

					//NTBs information
					OutVertexTangents[AddedVertexInstanceId] = Verts[Indexes[VertexInstanceIndex]].TangentX;
					OutVertexBinormalSigns[AddedVertexInstanceId] = GetBasisDeterminantSign(
						(FVector)Verts[Indexes[VertexInstanceIndex]].TangentX,
						(FVector)Verts[Indexes[VertexInstanceIndex]].TangentY,
						(FVector)Verts[Indexes[VertexInstanceIndex]].TangentZ);
					OutVertexNormals[AddedVertexInstanceId] = Verts[Indexes[VertexInstanceIndex]].TangentZ;

					//Vertex Color
					OutVertexColors[AddedVertexInstanceId] = Verts[Indexes[VertexInstanceIndex]].Color;

					//Texture coord
					for (int32 TexCoordIndex = 0; TexCoordIndex < InMeshNumTexCoords; TexCoordIndex++)
					{
						OutVertexUVs.Set(AddedVertexInstanceId, TexCoordIndex, Verts[Indexes[VertexInstanceIndex]].UVs[TexCoordIndex]);
					}
				}
				
				// material index
				int32 MaterialIndex = MaterialIndexes[TriangleIndex];
				FPolygonGroupID MaterialPolygonGroupID = INDEX_NONE;
				if (!PolygonGroupMapping.Contains(MaterialIndex))
				{
					FPolygonGroupID PolygonGroupID(MaterialIndex);
					check(InMesh.PolygonGroups().IsValid(PolygonGroupID));
					MaterialPolygonGroupID = OutReducedMesh.PolygonGroups().Num() > MaterialIndex ? PolygonGroupID : OutReducedMesh.CreatePolygonGroup();

					// Copy all attributes from the base polygon group to the new polygon group
					InMesh.PolygonGroupAttributes().ForEach(
						[&OutReducedMesh, PolygonGroupID, MaterialPolygonGroupID](const FName Name, const auto ArrayRef)
						{
							for (int32 Index = 0; Index < ArrayRef.GetNumChannels(); ++Index)
							{
								// Only copy shared attribute values, since input mesh description can differ from output mesh description
								const auto& Value = ArrayRef.Get(PolygonGroupID, Index);
								if (OutReducedMesh.PolygonGroupAttributes().HasAttribute(Name))
								{
									OutReducedMesh.PolygonGroupAttributes().SetAttribute(MaterialPolygonGroupID, Name, Index, Value);
								}
							}
						}
					);
					PolygonGroupMapping.Add(MaterialIndex, MaterialPolygonGroupID);
				}
				else
				{
					MaterialPolygonGroupID = PolygonGroupMapping[MaterialIndex];
				}

				// Insert a polygon into the mesh
				TArray<FEdgeID> NewEdgeIDs;
				const FTriangleID NewTriangleID = OutReducedMesh.CreateTriangle(MaterialPolygonGroupID, CornerInstanceIDs, &NewEdgeIDs);
				for (const FEdgeID& NewEdgeID : NewEdgeIDs)
				{
					// @todo: set NewEdgeID edge hardness?
				}
			}
			Verts.Empty();
			Indexes.Empty();

			//Remove the unused polygon group (reduce can remove all polygons from a group)
			TArray<FPolygonGroupID> ToDeletePolygonGroupIDs;
			for (const FPolygonGroupID PolygonGroupID : OutReducedMesh.PolygonGroups().GetElementIDs())
			{
				if (OutReducedMesh.GetPolygonGroupPolygonIDs(PolygonGroupID).Num() == 0)
				{
					ToDeletePolygonGroupIDs.Add(PolygonGroupID);
				}
			}
			for (const FPolygonGroupID& PolygonGroupID : ToDeletePolygonGroupIDs)
			{
				OutReducedMesh.DeletePolygonGroup(PolygonGroupID);
			}
		}
	}

	virtual bool ReduceSkeletalMesh(
		USkeletalMesh* SkeletalMesh,
		int32 LODIndex,
		const class ITargetPlatform* TargetPlatform
		) override
	{
		return false;
	}

	virtual bool IsSupported() const override
	{
		return true;
	}

	/**
	*	Returns true if mesh reduction is active. Active mean there will be a reduction of the vertices or triangle number
	*/
	virtual bool IsReductionActive(const struct FMeshReductionSettings &ReductionSettings) const
	{
		return IsReductionActive(ReductionSettings, 0, 0);
	}

	virtual bool IsReductionActive(const struct FMeshReductionSettings& ReductionSettings, uint32 NumVertices, uint32 NumTriangles) const
	{
		float Threshold_One = (1.0f - UE_KINDA_SMALL_NUMBER);
		switch (ReductionSettings.TerminationCriterion)
		{
		case EStaticMeshReductionTerimationCriterion::Triangles:
		{
			return ReductionSettings.PercentTriangles < Threshold_One || ReductionSettings.MaxNumOfTriangles < NumTriangles;
		}
		case EStaticMeshReductionTerimationCriterion::Vertices:
		{
			return ReductionSettings.PercentVertices < Threshold_One || ReductionSettings.MaxNumOfVerts < NumVertices;
		}
		case EStaticMeshReductionTerimationCriterion::Any:
		{
			return ReductionSettings.PercentTriangles < Threshold_One
				|| ReductionSettings.PercentVertices < Threshold_One
				|| ReductionSettings.MaxNumOfTriangles < NumTriangles
				|| ReductionSettings.MaxNumOfVerts < NumVertices;
		}
		}

		return false;
	}

	virtual bool IsReductionActive(const FSkeletalMeshOptimizationSettings &ReductionSettings) const
	{
		return false;
	}

	virtual bool IsReductionActive(const struct FSkeletalMeshOptimizationSettings &ReductionSettings, uint32 NumVertices, uint32 NumTriangles) const
	{
		return false;
	}

	virtual ~FQuadricSimplifierMeshReduction() {}

	static FQuadricSimplifierMeshReduction* Create()
	{
		return new FQuadricSimplifierMeshReduction;
	}
};

TUniquePtr<FQuadricSimplifierMeshReduction> GQuadricSimplifierMeshReduction;

void FQuadricSimplifierMeshReductionModule::StartupModule()
{
	GQuadricSimplifierMeshReduction.Reset(FQuadricSimplifierMeshReduction::Create());
	IModularFeatures::Get().RegisterModularFeature(IMeshReductionModule::GetModularFeatureName(), this);
}

void FQuadricSimplifierMeshReductionModule::ShutdownModule()
{
	GQuadricSimplifierMeshReduction = nullptr;
	IModularFeatures::Get().UnregisterModularFeature(IMeshReductionModule::GetModularFeatureName(), this);
}

IMeshReduction* FQuadricSimplifierMeshReductionModule::GetStaticMeshReductionInterface()
{
	return GQuadricSimplifierMeshReduction.Get();
}

IMeshReduction* FQuadricSimplifierMeshReductionModule::GetSkeletalMeshReductionInterface()
{
	return nullptr;
}

IMeshMerging* FQuadricSimplifierMeshReductionModule::GetMeshMergingInterface()
{
	return nullptr;
}

class IMeshMerging* FQuadricSimplifierMeshReductionModule::GetDistributedMeshMergingInterface()
{
	return nullptr;
}

FString FQuadricSimplifierMeshReductionModule::GetName()
{
	return FString("QuadricMeshReduction");	
}
