// Copyright (C) 2009 Nine Realms, Inc
//

#pragma once

#include "CoreMinimal.h"

/*
===============================================================================
	Unrolled Link List

	Link list of small arrays.
	blockSize must be power of 2
	Ordering is not maintained
===============================================================================
*/

template< typename T, uint32 blockSize >
class TUnrolledLinkList
{
	class TNode
	{
		friend class TUnrolledLinkList<T, blockSize>;
	public:
		TNode()
			: next(NULL)
			, prev(NULL)
		{}

		~TNode() { delete next; }

	private:
		T		block[blockSize];
		TNode*	next;
		TNode*	prev;
	};

public:
	class TIterator
	{
		friend class TUnrolledLinkList<T, blockSize>;
	public:
		TIterator()
			: node(NULL)
			, index(0)
		{}

		T&			operator*() { return node->block[ index & (blockSize - 1) ]; }
		T*			operator->() { return &node->block[ index & (blockSize - 1) ]; }

		bool		operator==( const TIterator& i ) const { return index == i.index; }
		bool		operator!=( const TIterator& i ) const { return index != i.index; }

		TIterator&	operator++()
		{
			checkSlow( node );
			index++;
			node = ( index & (blockSize - 1) ) ? node : node->next;
			return *this;
		}

	private:
		TIterator( TNode* n, uint32 i )
			: node(n)
			, index(i)
		{}

		TNode*	node;
		uint32	index;
	};

public:
				TUnrolledLinkList();

	uint32		Num() const		{ return num; }

	void		Add( const T& Element );
	void		Remove( const T& Element );
	void		Remove( const TIterator& i );

	void		Clear();

	TIterator	Begin()			{ return TIterator( &head, 0 ); }
	TIterator	End()			{ return TIterator( freeNode, num ); }

protected:
	TNode		head;
	TNode*		freeNode;	// first node that has a free element
	uint32		num;
};

template< typename T, uint32 blockSize >
FORCEINLINE TUnrolledLinkList<T, blockSize>::TUnrolledLinkList()
	: freeNode( &head )
	, num(0)
{
	static_assert((blockSize & (blockSize - 1)) == 0, "Block size must be power of 2.");
}

template< typename T, uint32 blockSize >
FORCEINLINE void TUnrolledLinkList<T, blockSize>::Add( const T& Element )
{
	if( num != 0 && ( num & (blockSize - 1) ) == 0 )
	{
		if( !freeNode->next )
		{
			freeNode->next = new TNode;
			freeNode->next->prev = freeNode;
		}
		
		freeNode = freeNode->next;
	}

	checkSlow( freeNode );

	freeNode->block[ num & (blockSize - 1) ] = Element;
	num++;
}

template< typename T, uint32 blockSize >
FORCEINLINE void TUnrolledLinkList<T, blockSize>::Remove( const T& Element )
{
	for( TIterator i = Begin(); i != End(); ++i )
	{
		if( *i == Element )
		{
			Remove( i );
			return;
		}
	}
}

template< typename T, uint32 blockSize >
FORCEINLINE void TUnrolledLinkList<T, blockSize>::Remove( const TIterator& i )
{
	checkSlow( num > 0 );
	checkSlow( i.index < num );

	num--;
	
	if( i.index != num )
	{
		// i isn't last element so copy last element into hole
		i.node->block[ i.index & (blockSize - 1) ] = freeNode->block[ num & (blockSize - 1) ];
	}
	
	if( ( num & (blockSize - 1) ) == 0 )
	{
		// more than one fully empty node after freeNode
		delete freeNode->next;
		freeNode->next = NULL;

		freeNode = freeNode->prev ? freeNode->prev : freeNode;
	}
}

template< typename T, uint32 blockSize >
FORCEINLINE void TUnrolledLinkList<T, blockSize>::Clear()
{
	delete head.next;
	head.next = NULL;
	head.prev = NULL;
	freeNode = &head;
	num = 0;
}
