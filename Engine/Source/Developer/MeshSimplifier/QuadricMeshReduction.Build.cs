// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class QuadricMeshReduction : ModuleRules
{
	public QuadricMeshReduction(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"RenderCore",
				"NaniteUtilities",
				"MeshUtilitiesCommon",
				"MeshDescription",
				"StaticMeshDescription",
			}
		);

        PrivateIncludePathModuleNames.AddRange(
        new string[] {
                "MeshReductionInterface",
             }
        );

        PublicIncludePaths.Add("Developer/MeshSimplifier/Private");
        //PrivateIncludePaths.Add("Developer/MeshSimplifier/Private");
    }
}
